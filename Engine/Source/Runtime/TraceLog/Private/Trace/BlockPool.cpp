// Copyright Epic Games, Inc. All Rights Reserved.

#include "Trace/Config.h"

#if TRACE_PRIVATE_MINIMAL_ENABLED

#include "Platform.h"
#include "Misc/ScopeExit.h"
#include "Trace/Detail/Atomic.h"
#include "Trace/Detail/Writer.inl"

namespace UE {
namespace Trace {
namespace Private {

////////////////////////////////////////////////////////////////////////////////
void*		Writer_MemoryAllocate(SIZE_T, uint32);
void		Writer_MemoryFree(void*, uint32);
void		LogStall(uint64, uint64);

extern FStatistics GTraceStatistics;

////////////////////////////////////////////////////////////////////////////////
struct FPoolPage
{
	FPoolPage*	NextPage;
	uint32		AllocSize;
};

////////////////////////////////////////////////////////////////////////////////
struct FPoolBlockList
{
	FWriteBuffer*	Head;
	FWriteBuffer*	Tail;
};



////////////////////////////////////////////////////////////////////////////////
#define T_ALIGN alignas(PLATFORM_CACHE_LINE_SIZE)
static const uint32						GPoolBlockSize		= UE_TRACE_BLOCK_SIZE;
static const uint32						GPoolPageSize		= GPoolBlockSize << 4;
static const uint32						GPoolInitPageSize	= GPoolBlockSize << 6;
T_ALIGN static volatile uint32			GPoolTotalAllocated;// = 0
static uint32							GPoolMaxSize		= ~0; // Starts unlimited, written by initialization
T_ALIGN static FWriteBuffer* volatile	GPoolFreeList;		// = nullptr;
T_ALIGN static UPTRINT volatile			GPoolFutex;			// = 0
T_ALIGN static FPoolPage* volatile		GPoolPageList;		// = nullptr;
static const uint32						GPoolLimitTimeoutMs	= 4 * 1000;
#undef T_ALIGN

////////////////////////////////////////////////////////////////////////////////
void Writer_SetBlockPoolLimit(uint32 MaxSize)
{
	/*
	 * Note that setting it higher than the default value introduces risks that
	 * serialized events will be transmitted in the wrong order. The smallest 
	 * possible size for a (serialized) event, one with no field, is 5 bytes.
	 * Serials wrap every 16M event. This gives a theoretical limit of 79MiB.
	 */
	GPoolMaxSize = MaxSize; 
}

////////////////////////////////////////////////////////////////////////////////
void Writer_UnsetBlockPoolLimit()
{
	GPoolMaxSize = ~0;
}

////////////////////////////////////////////////////////////////////////////////
static FPoolBlockList Writer_AllocateBlockList(uint32 PageSize, uint32& OutBlockCount)
{
	// The free list is empty so we have to populate it with some new blocks.
	uint8* PageBase = (uint8*)Writer_MemoryAllocate(PageSize, PLATFORM_CACHE_LINE_SIZE);
	
#if TRACE_PRIVATE_STATISTICS
	AtomicAddRelaxed(&GTraceStatistics.BlockPoolAllocated, uint64(PageSize));
#endif
	AtomicAddRelaxed(&GPoolTotalAllocated, PageSize);

	uint32 BufferSize = GPoolBlockSize;
	BufferSize -= sizeof(FWriteBuffer);
	BufferSize -= sizeof(uint32); // to preceed event data with a small header when sending.

	// Link subsequent blocks together
	uint8* FirstBlock = PageBase + GPoolBlockSize - sizeof(FWriteBuffer);
	uint8* Block = FirstBlock;
	const uint32 BlockCount = PageSize / GPoolBlockSize;
	for (int i = 1, n = BlockCount; ; ++i)
	{
		auto* Buffer = (FWriteBuffer*)Block;
		Buffer->Size = uint16(BufferSize);
		Buffer->CursorMaskInv = 0;
		if (i >= n)
		{
			break;
		}

		AtomicStoreRelaxed(&(Buffer->NextBuffer), (FWriteBuffer*)(Block + GPoolBlockSize));
		Block += GPoolBlockSize;
	}

	AtomicAddRelaxed(&GTraceStatistics.BlockPoolAllocatedBlocks, BlockCount);
	OutBlockCount = BlockCount;

	// Keep track of allocation base so we can free it on shutdown
	FWriteBuffer* NextBuffer = (FWriteBuffer*)FirstBlock;
	NextBuffer->Size -= sizeof(FPoolPage);
	FPoolPage* PageListNode = (FPoolPage*)PageBase;
	PageListNode->NextPage = GPoolPageList;
	PageListNode->AllocSize = PageSize;
	GPoolPageList = PageListNode;

	return { NextBuffer, (FWriteBuffer*)Block };
}

////////////////////////////////////////////////////////////////////////////////
FWriteBuffer* Writer_GetPanicPool()
{
	static FWriteBuffer* SPanicBuffer = []()
	{
		uint8 *Allocation = (uint8 *)Writer_MemoryAllocate(
			GPoolBlockSize, PLATFORM_CACHE_LINE_SIZE);
#if TRACE_PRIVATE_STATISTICS
		AtomicAddRelaxed(&GTraceStatistics.BlockPoolAllocated, uint64(GPoolBlockSize));
#endif
		uint32 BufferSize = GPoolBlockSize - sizeof(FWriteBuffer);
		FWriteBuffer* PanicBuffer = (FWriteBuffer *)(Allocation + BufferSize);
		static_assert(GPoolBlockSize < UINT16_MAX);
		PanicBuffer->CursorMaskInv = 0;
		PanicBuffer->Size = uint16(BufferSize);
		PanicBuffer->NextBuffer = nullptr;
		PanicBuffer->NextThread = nullptr;
		PanicBuffer->SetCursor(Allocation);
		PanicBuffer->Reaped = Allocation;
		PanicBuffer->EtxOffset = 0 - int32(sizeof(FWriteBuffer));
		// This mask will cause the cursor to stay at the start of
		// the buffer, without ever increasing
		PanicBuffer->CursorMaskInv = ~0;
		return PanicBuffer;
	}();
	return SPanicBuffer;
}

////////////////////////////////////////////////////////////////////////////////
FWriteBuffer* Writer_AllocateBlockFromPool()
{
	int32 ThrottleRestore = -1;
	ON_SCOPE_EXIT {
		if (ThrottleRestore >= 0)
		{
			ThreadUnthrottle(ThrottleRestore);
		}
	};

	auto ApplyThrottle = [&ThrottleRestore, Count=uint32(0)] () mutable {
		if (ThrottleRestore < 0 && Count++ > 0)
		{
			ThrottleRestore = ThreadThrottle();
		}
	};

	// Fetch a new buffer
	uint64 StallStart = 0;
	FWriteBuffer* Ret;
	while (true)
	{
		ApplyThrottle();

		// First we'll try one from the free list
		FWriteBuffer* Owned = AtomicLoadRelaxed(&GPoolFreeList);
		if (Owned != nullptr)
		{
			FWriteBuffer* OwnedNext = AtomicLoadRelaxed(&(Owned->NextBuffer));

			if (!AtomicCompareExchangeAcquire(&GPoolFreeList, OwnedNext, Owned))
			{
				PlatformYield();
				continue;
			}
		}

		// If we didn't fetch the sentinal then we've taken a block we can use
		if (Owned != nullptr)
		{
			AtomicSubRelaxed(&GTraceStatistics.BlockPoolFreeBlocks, 1u);
			Ret = (FWriteBuffer*)Owned;
			break;
		}

		// If we have hit the pool limit, continue looping until the worker
		// thread has had time write out and give back free buffers.
		
		if ((AtomicLoadAcquire(&GPoolTotalAllocated) >= GPoolMaxSize))
		{
			const uint64 Ts = TimeGetRelativeTimestamp();
			StallStart = StallStart ? StallStart : Ts;
			uint32 DurationMs = uint32(1000.f*(float(Ts - StallStart) / float(TimeGetFrequency())));
			if (DurationMs > GPoolLimitTimeoutMs)
			{
				// At this point we have timed out waiting for a free block.
				// This likely means the worker thread cannot keep up with
				// tracing bandwidth or that I/O is stalled.
				Panic();
				Ret = Writer_GetPanicPool();
				break;
			}
			continue;
		}

		// The free list is empty. Map some more memory.
		UPTRINT Futex = AtomicLoadRelaxed(&GPoolFutex);
		if (Futex || !AtomicCompareExchangeAcquire(&GPoolFutex, Futex + 1, Futex))
		{
			// Someone else is mapping memory so we'll briefly yield and try the
			// free list again.
			ThreadSleep(0);
			continue;
		}

		uint32 BlockCount = 0;
		FPoolBlockList BlockList = Writer_AllocateBlockList(GPoolPageSize, BlockCount);
		Ret = BlockList.Head;

		// And insert the block list into the freelist. 'Block' is now the last block
		for (auto* ListNode = BlockList.Tail;; PlatformYield())
		{
			FWriteBuffer* FreeListValue = AtomicLoadRelaxed(&GPoolFreeList);
			AtomicStoreRelaxed(&(ListNode->NextBuffer), FreeListValue);

			if (AtomicCompareExchangeRelease(&GPoolFreeList, Ret->NextBuffer, FreeListValue))
			{
				AtomicAddRelaxed(&GTraceStatistics.BlockPoolFreeBlocks, BlockCount - 1);
				
				break;
			}
		}

		// Let other threads proceed. They should hopefully hit the free list
		for (;; Private::PlatformYield())
		{
			if (AtomicCompareExchangeRelease<UPTRINT>(&GPoolFutex, 0, 1))
			{
				break;
			}
		}

		break;
	}

	if (StallStart)
	{
		const uint64 StallEnd = TimeGetRelativeTimestamp();
		LogStall(StallStart, StallEnd);
	}

	return Ret;
}

////////////////////////////////////////////////////////////////////////////////
void Writer_FreeBlockListToPool(FWriteBuffer* Head, FWriteBuffer* Tail)
{
	for (FWriteBuffer* ListNode = Tail;; PlatformYield())
	{
		FWriteBuffer* FreeListValue = AtomicLoadRelaxed(&GPoolFreeList);
		AtomicStoreRelaxed(&(ListNode->NextBuffer), FreeListValue);

		if (AtomicCompareExchangeRelease(&GPoolFreeList, Head, FreeListValue))
		{
			break;
		}
	}
}

////////////////////////////////////////////////////////////////////////////////
void Writer_InitializePool()
{
	static_assert(GPoolPageSize >= 0x10000, "Page growth must be >= 64KB");
	static_assert(GPoolInitPageSize >= 0x10000, "Initial page size must be >= 64KB");
}

////////////////////////////////////////////////////////////////////////////////
void Writer_ShutdownPool()
{
	// Claim ownership of the pool page list. There really should be no one
	// creating so we'll just read it an go instead of a CAS loop.
	for (auto* Page = AtomicLoadRelaxed(&GPoolPageList); Page != nullptr;)
	{
		FPoolPage* NextPage = Page->NextPage;
#if TRACE_PRIVATE_STATISTICS
		AtomicSubRelease(&GTraceStatistics.BlockPoolAllocated, uint64(Page->AllocSize));
#endif
		AtomicSubRelease(&GPoolTotalAllocated, Page->AllocSize);
		Writer_MemoryFree(Page, Page->AllocSize);
		Page = NextPage;
	}
}

} // namespace Private
} // namespace Trace
} // namespace UE

#endif // TRACE_PRIVATE_MINIMAL_ENABLED
