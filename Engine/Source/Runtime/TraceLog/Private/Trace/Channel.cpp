// Copyright Epic Games, Inc. All Rights Reserved.

#include "Trace/Detail/Channel.h"

#include "Math/UnrealMathUtility.h"
#include "Trace/Detail/Atomic.h"
#include "Trace/Detail/Channel.h"
#include "Trace/Lock.h"
#include "Trace/Trace.h"
#include "Trace/Trace.inl"

#include <ctype.h>

#if TRACE_PRIVATE_MINIMAL_ENABLED

namespace UE {
namespace Trace {

////////////////////////////////////////////////////////////////////////////////
struct FTraceChannel : public FChannel
{
	bool IsEnabled() const { return true; }
	explicit operator bool() const { return true; }
};

static FTraceChannel	TraceLogChannelDetail;
FChannel&				TraceLogChannel			= TraceLogChannelDetail;

#if TRACE_PRIVATE_FULL_ENABLED

///////////////////////////////////////////////////////////////////////////////
UE_TRACE_EVENT_BEGIN(Trace, ChannelAnnounce, NoSync|Important)
	UE_TRACE_EVENT_FIELD(uint32, Id)
	UE_TRACE_EVENT_FIELD(bool, IsEnabled)
	UE_TRACE_EVENT_FIELD(bool, ReadOnly)
	UE_TRACE_EVENT_FIELD(AnsiString, Name)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Trace, ChannelToggle, NoSync|Important)
	UE_TRACE_EVENT_FIELD(uint32, Id)
	UE_TRACE_EVENT_FIELD(bool, IsEnabled)
UE_TRACE_EVENT_END()

#endif // TRACE_PRIVATE_FULL_ENABLED

///////////////////////////////////////////////////////////////////////////////
static FChannel* volatile	GHeadChannel;			// = nullptr;
static FChannel* volatile	GNewChannelList;		// = nullptr;
static bool 				GChannelsInitialized;

////////////////////////////////////////////////////////////////////////////////
static uint32 GetChannelHash(const ANSICHAR* Input, int32 Length)
{
	// Make channel names tolerant to ending 's' (or 'S').
	// Example: "Log", "log", "logs", "LOGS" and "LogsChannel" will all match as being the same channel.
	if (Length > 0 && (Input[Length - 1] | 0x20) == 's')
	{
		--Length;
	}

	uint32 Result = 0x811c9dc5;
	for (; Length; ++Input, --Length)
	{
		Result ^= *Input | 0x20; // a cheap ASCII-only case insensitivity.
		Result *= 0x01000193;
	}
	return Result;
}

///////////////////////////////////////////////////////////////////////////////
static uint32 GetChannelNameLength(const ANSICHAR* ChannelName)
{
	// Strip "Channel" suffix if it exists
	size_t Len = uint32(strlen(ChannelName));
	if (Len > 7)
	{
		if (strcmp(ChannelName + Len - 7, "Channel") == 0)
		{
			Len -= 7;
		}
	}

	return uint32(Len);
}



///////////////////////////////////////////////////////////////////////////////
FChannel::Iter::~Iter()
{
	if (Inner[2] == nullptr)
	{
		return;
	}

	using namespace Private;
	for (auto* Node = (FChannel*)Inner[2];; PlatformYield())
	{
		FChannel* OldHead = AtomicLoadRelaxed(&GHeadChannel);
		AtomicStoreRelaxed(&(Node->Next), OldHead);
		if (AtomicCompareExchangeRelaxed(&GHeadChannel, (FChannel*)Inner[1], OldHead))
		{
			break;
		}
	}
}

///////////////////////////////////////////////////////////////////////////////
const FChannel* FChannel::Iter::GetNext()
{
	using namespace Private;

	auto* Ret = (FChannel*)Inner[0];
	if (Ret != nullptr)
	{
		Inner[0] = AtomicLoadRelaxed(&Ret->Next);
		if (Inner[0] != nullptr)
		{
			Inner[2] = Inner[0];
		}
	}
	return Ret;
}



///////////////////////////////////////////////////////////////////////////////
FChannel::Iter FChannel::ReadNew()
{
	using namespace Private;

	FChannel* List = AtomicLoadRelaxed(&GNewChannelList);
	if (List == nullptr)
	{
		return {};
	}

	while (!AtomicCompareExchangeAcquire(&GNewChannelList, (FChannel*)nullptr, List))
	{
		PlatformYield();
		List = AtomicLoadRelaxed(&GNewChannelList);
	}

	return { { List, List, List } };
}

///////////////////////////////////////////////////////////////////////////////
void FChannel::Setup(const ANSICHAR* InChannelName, const InitArgs& InArgs)
{
	using namespace Private;

	Name.Ptr = InChannelName;
	Name.Len = GetChannelNameLength(Name.Ptr);
	Name.Hash = GetChannelHash(Name.Ptr, Name.Len);
	Args = InArgs;

	// Append channel to the linked list of new channels.
	for (;; PlatformYield())
	{
		FChannel* HeadChannel = AtomicLoadRelaxed(&GNewChannelList);
		AtomicStoreRelaxed(&Next, HeadChannel);
		if (AtomicCompareExchangeRelease(&GNewChannelList, this, Next))
		{
			break;
		}
	}

	// If channel is initialized after all channels are disabled (post static init)
	// this channel needs to be disabled too.
	if (GChannelsInitialized)
	{
		Enabled = -1;
	}
}

///////////////////////////////////////////////////////////////////////////////
void FChannel::Announce() const
{
#if TRACE_PRIVATE_FULL_ENABLED
	UE_TRACE_LOG(Trace, ChannelAnnounce, TraceLogChannel, Name.Len * sizeof(ANSICHAR))
		<< ChannelAnnounce.Id(Name.Hash)
		<< ChannelAnnounce.IsEnabled(IsEnabled())
		<< ChannelAnnounce.ReadOnly(Args.bReadOnly)
		<< ChannelAnnounce.Name(Name.Ptr, Name.Len);
#endif // TRACE_PRIVATE_FULL_ENABLED
}

///////////////////////////////////////////////////////////////////////////////
void FChannel::Initialize()
{
	// During static initialization, all channels are created as enabled (zero),
	// and act like so from the process start until this method is called (i.e. when Trace is initialized).
	// Now we can disable all channels.
	// Channels specified on the command line (using -trace=<channels> argument)
	// will be further re-enabled after this call.
	ToggleAllInternal(false, false);
	GChannelsInitialized = true;
}

///////////////////////////////////////////////////////////////////////////////
void FChannel::ToggleAll(bool bEnabled)
{
	ToggleAllInternal(bEnabled, true);
}

///////////////////////////////////////////////////////////////////////////////
void FChannel::ToggleAllInternal(bool bEnabled, bool bIssueCallback)
{
	using namespace Private;

	FChannel* ChannelLists[] =
	{
		AtomicLoadAcquire(&GNewChannelList),
		AtomicLoadAcquire(&GHeadChannel),
	};
	for (FChannel* Channel : ChannelLists)
	{
		for (; Channel != nullptr; Channel = AtomicLoadRelaxed(&Channel->Next))
		{
			Channel->ToggleInternal(bEnabled, bIssueCallback);
		}
	}
}

///////////////////////////////////////////////////////////////////////////////
void FChannel::PanicDisableAll()
{
	using namespace Private;

	FChannel* ChannelLists[] =
	{
		AtomicLoadAcquire(&GNewChannelList),
		AtomicLoadAcquire(&GHeadChannel),
	};
	for (FChannel* Channel : ChannelLists)
	{
		for (; Channel != nullptr; Channel = AtomicLoadRelaxed(&Channel->Next))
		{
			AtomicStoreRelaxed(&Channel->Enabled, -1);
		}
	}
}

///////////////////////////////////////////////////////////////////////////////
FChannel* FChannel::FindChannel(const ANSICHAR* ChannelName)
{
	using namespace Private;

	const uint32 ChannelNameLen = GetChannelNameLength(ChannelName);
	const uint32 ChannelNameHash = GetChannelHash(ChannelName, ChannelNameLen);

	FChannel* ChannelLists[] =
	{
		AtomicLoadAcquire(&GNewChannelList),
		AtomicLoadAcquire(&GHeadChannel),
	};
	for (FChannel* Channel : ChannelLists)
	{
		for (; Channel != nullptr; Channel = AtomicLoadRelaxed(&Channel->Next))
		{
			if (Channel->Name.Hash == ChannelNameHash)
			{
				return Channel;
			}
		}
	}

	return nullptr;
}
	
///////////////////////////////////////////////////////////////////////////////
FChannel* FChannel::FindChannel(FChannelId ChannelId)
{
	using namespace Private;

	FChannel* ChannelLists[] =
	{
		AtomicLoadAcquire(&GNewChannelList),
		AtomicLoadAcquire(&GHeadChannel),
	};
	for (FChannel* Channel : ChannelLists)
	{
		for (; Channel != nullptr; Channel = AtomicLoadRelaxed(&Channel->Next))
		{
			if (Channel->Name.Hash == ChannelId)
			{
				return Channel;
			}
		}
	}

	return nullptr;
}

///////////////////////////////////////////////////////////////////////////////
void FChannel::EnumerateChannels(ChannelIterCallback Func, void* User)
{
	using namespace Private;
	FChannel* ChannelLists[] =
	{
		AtomicLoadAcquire(&GNewChannelList),
		AtomicLoadAcquire(&GHeadChannel),
	};

	FChannelInfo Info;
	for (FChannel* Channel : ChannelLists)
	{
		for (; Channel != nullptr; Channel = AtomicLoadRelaxed(&Channel->Next))
		{
			Info.Name = Channel->Name.Ptr;
			Info.Desc = Channel->Args.Desc;
			Info.bIsEnabled = Channel->IsEnabled();
			Info.bIsReadOnly = Channel->Args.bReadOnly;
			Info.Id = Channel->Name.Hash;
			bool Result = Func(Info, User);
			if (!Result)
			{
				return;
			}
		}
	}
}
	
///////////////////////////////////////////////////////////////////////////////
bool FChannel::Toggle(bool bEnabled, const TCHAR** OutReason)
{
	return ToggleInternal(bEnabled, true, OutReason);
}

///////////////////////////////////////////////////////////////////////////////
bool FChannel::ToggleInternal(bool bEnabled, bool bIssueCallback, const TCHAR** OutReason)
{
	using namespace Private;

	Private::FLock ScopedLock(&Lock);
	if (ScopedLock.TimedOut())
	{
		static const TCHAR* TimedOutError = TEXT("Timed out waiting for lock");
		if (OutReason)
		{
			*OutReason = TimedOutError;
		}
		return IsEnabled();
	}

	bool bCanToggle = bIssueCallback ? OnToggle(bEnabled, OutReason) : true;
	if (bCanToggle)
	{
		AtomicStoreRelaxed(&Enabled, bEnabled ? 1 : -1);

#if TRACE_PRIVATE_FULL_ENABLED
		UE_TRACE_LOG(Trace, ChannelToggle, TraceLogChannel)
			<< ChannelToggle.Id(Name.Hash)
			<< ChannelToggle.IsEnabled(IsEnabled());
#endif // TRACE_PRIVATE_FULL_ENABLED

		OnToggled(bEnabled);
	}

	return IsEnabled();
}

///////////////////////////////////////////////////////////////////////////////
bool FChannel::Toggle(const ANSICHAR* ChannelName, bool bEnabled)
{
	if (FChannel* Channel = FChannel::FindChannel(ChannelName))
	{
		return Channel->ToggleInternal(bEnabled, true);
	}
	return false;
}

} // namespace Trace
} // namespace UE

#endif // TRACE_PRIVATE_MINIMAL_ENABLED
