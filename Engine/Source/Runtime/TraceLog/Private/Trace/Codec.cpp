// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "Platform.h"
#include "Message.h"

THIRD_PARTY_INCLUDES_START
#if defined(_MSC_VER)
#	pragma warning(push)
#	pragma warning(disable : 6239)
#endif

#if !defined(TRACE_PRIVATE_EXTERNAL_LZ4)
#	define LZ4_NAMESPACE Trace
#		include "LZ4/lz4.c.inl"
#	undef LZ4_NAMESPACE
#	define TRACE_PRIVATE_LZ4_NAMESPACE ::Trace::
#else
#	define TRACE_PRIVATE_LZ4_NAMESPACE
#endif

#if defined(_MSC_VER)
#	pragma warning(pop)
#endif
THIRD_PARTY_INCLUDES_END

namespace UE {
namespace Trace {
namespace Private {

////////////////////////////////////////////////////////////////////////////////
TRACELOG_API uint32 GetEncodeMaxSize(uint32 InputSize)
{
	// Guard against 7 byte overshoot from LZ4_wildCopy8
	return LZ4_COMPRESSBOUND(InputSize) + WILDCOPYLENGTH;
}

////////////////////////////////////////////////////////////////////////////////
TRACELOG_API int32 Encode(const void* Src, int32 SrcSize, void* Dest, int32 DestSize)
{
	constexpr int Acceleration = 1; // increase by 1 for small speed increase
	return TRACE_PRIVATE_LZ4_NAMESPACE LZ4_compress_fast((const char*)Src, (char*)Dest, SrcSize, DestSize, Acceleration);
}

////////////////////////////////////////////////////////////////////////////////
int32 EncodeInstr(const void* Src, int32 SrcSize, void* Dest, int32 DestSize)
{
#if TRACE_PRIVATE_MINIMAL_ENABLED
	FProfilerScope _(__func__);
#endif
	int Result = Encode(Src, SrcSize, Dest, DestSize);
#if UE_TRACE_ENABLED
	if (!Result)
	{
		UE_TRACE_MESSAGE_F(CompressionError, "LZ4 failed to compress %d bytes (dest %d bytes).", SrcSize, DestSize);
	}
#endif
	return Result;
}

////////////////////////////////////////////////////////////////////////////////
TRACELOG_API int32 Decode(const void* Src, int32 SrcSize, void* Dest, int32 DestSize)
{
	return TRACE_PRIVATE_LZ4_NAMESPACE LZ4_decompress_safe((const char*)Src, (char*)Dest, SrcSize, DestSize);
}

} // namespace Private
} // namespace Trace
} // namespace UE
