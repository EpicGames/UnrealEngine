// Copyright Epic Games, Inc. All Rights Reserved.

#include "Trace/Config.h"

#include "Trace/Platform.h"
#include "Trace/Message.h"
#include "Trace/Detail/Channel.h"

#include "Misc/CString.h"
#include "Templates/UnrealTemplate.h"

#include <type_traits>

namespace UE {
namespace Trace {
namespace Private {

#if TRACE_PRIVATE_ALLOW_TCP_CONTROL

////////////////////////////////////////////////////////////////////////////////
bool	Writer_SendTo(const ANSICHAR*, uint32=0, uint32=0);
bool	Writer_WriteTo(const ANSICHAR*, uint32=0);
bool	Writer_Stop();



////////////////////////////////////////////////////////////////////////////////
enum class EControlState : uint8
{
	Closed = 0,
	Listening,
	Accepted,
	Failed,
};

////////////////////////////////////////////////////////////////////////////////
struct FControlCommands
{
	enum { Max = 8 };
	struct
	{
		uint32	Hash;
		void*	Param;
		void	(*Thunk)(void*, uint32, ANSICHAR const* const*);
	}			Commands[Max];
	uint8		Count;
};
static_assert(std::is_trivial<FControlCommands>(), "FControlCommands must be trivial");



////////////////////////////////////////////////////////////////////////////////
static FControlCommands	GControlCommands;
static UPTRINT			GControlListen		= 0;
static UPTRINT			GControlSocket		= 0;
static EControlState	GControlState;		// = EControlState::Closed;
static uint16			GControlPort		= 1985;

////////////////////////////////////////////////////////////////////////////////
static uint32 Writer_ControlHash(const ANSICHAR* Word)
{
	uint32 Hash = 5381;
	for (; *Word; (Hash = (Hash * 33) ^ *Word), ++Word);
	return Hash;
}

////////////////////////////////////////////////////////////////////////////////
static bool Writer_ControlAddCommand(
	const ANSICHAR* Name,
	void* Param,
	void (*Thunk)(void*, uint32, ANSICHAR const* const*))
{
	if (GControlCommands.Count >= FControlCommands::Max)
	{
		return false;
	}

	uint32 Index = GControlCommands.Count++;
	GControlCommands.Commands[Index] = { Writer_ControlHash(Name), Param, Thunk };
	return true;
}

////////////////////////////////////////////////////////////////////////////////
static bool Writer_ControlDispatch(uint32 ArgC, ANSICHAR const* const* ArgV)
{
	if (ArgC == 0)
	{
		return false;
	}

	uint32 Hash = Writer_ControlHash(ArgV[0]);
	--ArgC;
	++ArgV;

	for (int i = 0, n = GControlCommands.Count; i < n; ++i)
	{
		const auto& Command = GControlCommands.Commands[i];
		if (Command.Hash == Hash)
		{
			Command.Thunk(Command.Param, ArgC, ArgV);
			return true;
		}
	}

	return false;
}

////////////////////////////////////////////////////////////////////////////////
static bool Writer_ControlListen()
{
	GControlListen = TcpSocketListen(GControlPort);
	if (!GControlListen)
	{
		uint32 Seed = uint32(TimeGetTimestamp());
		for (uint32 i = 0; i < 10 && !GControlListen; Seed *= 13, ++i)
		{
			uint16 Port((Seed & 0x1fff) + 0x8000);
			GControlListen = TcpSocketListen(Port);
			if (GControlListen)
			{
				GControlPort = Port;
				break;
			}
		}
	}

	if (!GControlListen)
	{
		//This unfortunately triggers on editor shutdown needlessly spamming the log
		//UE_TRACE_ERRORMESSAGE_F(ListenFail, GetLastErrorCode(), "Port: %d", GControlPort);
		GControlState = EControlState::Failed;
		return false;
	}

	UE_TRACE_MESSAGE_F(Display, "Control listening on port %u", GControlPort);
	GControlState = EControlState::Listening;
	return true;
}

////////////////////////////////////////////////////////////////////////////////
static bool Writer_ControlAccept()
{
	UPTRINT Socket;
	int Return = TcpSocketAccept(GControlListen, Socket);
	if (Return <= 0)
	{
		if (Return == -1)
		{
			IoClose(GControlListen);
			GControlListen = 0;
			GControlState = EControlState::Failed;
		}
		return false;
	}

	GControlState = EControlState::Accepted;
	GControlSocket = Socket;
	return true;
}

////////////////////////////////////////////////////////////////////////////////
static void Writer_ControlRecv()
{
	// We'll assume that commands are smaller than the canonical MTU so this
	// doesn't need to be implemented in a reentrant manner (maybe).

	ANSICHAR Buffer[512];
	ANSICHAR* __restrict Head = Buffer;
	while (TcpSocketHasData(GControlSocket))
	{
		int32 ReadSize = int32(UPTRINT(Buffer + sizeof(Buffer) - Head));
		int32 Recvd = IoRead(GControlSocket, Head, ReadSize);
		if (Recvd <= 0)
		{
			IoClose(GControlSocket);
			GControlSocket = 0;
			GControlState = EControlState::Listening;
			break;
		}

		Head += Recvd;

		enum EParseState
		{
			CrLfSkip,
			WhitespaceSkip,
			Word,
		} ParseState = EParseState::CrLfSkip;

		uint32 ArgC = 0;
		const ANSICHAR* ArgV[16];

		const ANSICHAR* __restrict Spent = Buffer;
		for (ANSICHAR* __restrict Cursor = Buffer; Cursor < Head; ++Cursor)
		{
			switch (ParseState)
			{
			case EParseState::CrLfSkip:
				if (*Cursor == '\n' || *Cursor == '\r')
				{
					continue;
				}
				ParseState = EParseState::WhitespaceSkip;
				/* [[fallthrough]] */

			case EParseState::WhitespaceSkip:
				if (*Cursor == ' ' || *Cursor == '\0')
				{
					continue;
				}

				if (ArgC < UE_ARRAY_COUNT(ArgV))
				{
					ArgV[ArgC] = Cursor;
					++ArgC;
				}

				ParseState = EParseState::Word;
				/* [[fallthrough]] */

			case EParseState::Word:
				if (*Cursor == ' ' || *Cursor == '\0')
				{
					*Cursor = '\0';
					ParseState = EParseState::WhitespaceSkip;
					continue;
				}

				if (*Cursor == '\r' || *Cursor == '\n')
				{
					*Cursor = '\0';

					Writer_ControlDispatch(ArgC, ArgV);

					ArgC = 0;
					Spent = Cursor + 1;
					ParseState = EParseState::CrLfSkip;
					continue;
				}

				break;
			}
		}

		int32 UnspentSize = int32(UPTRINT(Head - Spent));
		if (UnspentSize)
		{
			memmove(Buffer, Spent, UnspentSize);
		}
		Head = Buffer + UnspentSize;
	}
}

////////////////////////////////////////////////////////////////////////////////
uint32 Writer_GetControlPort()
{
	return GControlPort;
}

////////////////////////////////////////////////////////////////////////////////
void Writer_UpdateControl()
{
	switch (GControlState)
	{
	case EControlState::Closed:
		if (!Writer_ControlListen())
		{
			break;
		}
		/* [[fallthrough]] */

	case EControlState::Listening:
		if (!Writer_ControlAccept())
		{
			break;
		}
		/* [[fallthrough]] */

	case EControlState::Accepted:
		Writer_ControlRecv();
		break;
	}
}

////////////////////////////////////////////////////////////////////////////////
void Writer_InitializeControl()
{
	Writer_ControlAddCommand("SendTo", nullptr,
		[] (void*, uint32 ArgC, ANSICHAR const* const* ArgV)
		{
			if (ArgC > 0)
			{
				const ANSICHAR* Host = ArgV[0];
				uint32 Port = 0;
				uint32 Flags = 0;

				constexpr SIZE_T HostNameBufferSize = 256;
				ANSICHAR HostNameBuffer[HostNameBufferSize] = {};
				const ANSICHAR* PortStr = FPlatformString::Strchr(Host, ':');
				if (PortStr)
				{
					++PortStr;
					SIZE_T MaxLen = SIZE_T(PTRINT(PortStr - Host));
					if (MaxLen > HostNameBufferSize)
					{
						MaxLen = HostNameBufferSize;
					}
PRAGMA_DISABLE_DEPRECATION_WARNINGS
					// We can't use FPlatformString::Strncpy because of circular module dependencies
					strncpy(HostNameBuffer, ArgV[0], MaxLen);
					HostNameBuffer[MaxLen - 1] = '\0';
PRAGMA_ENABLE_DEPRECATION_WARNINGS
					Host = HostNameBuffer;
					Port = uint32(FPlatformString::Atoi(PortStr));
				}

				Writer_SendTo(Host, Flags, Port);
			}
		}
	);

	Writer_ControlAddCommand("Stop", nullptr,
		[] (void*, uint32 ArgC, ANSICHAR const* const* ArgV)
		{
			Writer_Stop();
		}
	);

	Writer_ControlAddCommand("ToggleChannels", nullptr,
		[] (void*, uint32 ArgC, ANSICHAR const* const* ArgV)
		{
			if (ArgC < 2)
			{
				return;
			}

			const size_t BufferSize = 512;
			ANSICHAR Channels[BufferSize] = {};
			ANSICHAR* Ctx;
			const bool bState = (ArgV[1][0] != '0');
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			// We can't use FPlatformString::Strncpy because of circular module dependencies
			strncpy(Channels, ArgV[0], BufferSize);
			Channels[BufferSize - 1] = '\0';
PRAGMA_ENABLE_DEPRECATION_WARNINGS
			ANSICHAR* Channel = FCStringAnsi::Strtok(Channels, ",", &Ctx);
			while (Channel)
			{
				FChannel::Toggle(Channel, bState);
				Channel = FCStringAnsi::Strtok(nullptr, ",", &Ctx);
			}
		}
	);
}

////////////////////////////////////////////////////////////////////////////////
void Writer_ShutdownControl()
{
	if (GControlListen)
	{
		IoClose(GControlListen);
		GControlListen = 0;
	}
}

#else

void	Writer_InitializeControl()	{}
void	Writer_ShutdownControl()	{}
void	Writer_UpdateControl()		{}
uint32	Writer_GetControlPort()		{ return ~0u; }
	
#endif // TRACE_PRIVATE_ALLOW_TCP_CONTROL

} // namespace Private
} // namespace Trace
} // namespace UE
