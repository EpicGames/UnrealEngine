// Copyright Epic Games, Inc. All Rights Reserved.

#include "Trace/Config.h"

#if TRACE_PRIVATE_MINIMAL_ENABLED && PLATFORM_UNIX

#include <arpa/inet.h>
#include <cstring>
#include <errno.h>
#include <fcntl.h>
#include <netdb.h>
#include <pthread.h>
#include <sys/mman.h>
#include <sys/socket.h>
#include <sys/types.h>
#include <time.h>
#include <unistd.h>
#if defined(_GNU_SOURCE)
	#include <sys/syscall.h>
#endif // _GNU_SOURCE

namespace UE {
namespace Trace {
namespace Private {

////////////////////////////////////////////////////////////////////////////////
UPTRINT ThreadCreate(const ANSICHAR* Name, void (*Entry)())
{
	void* (*PthreadThunk)(void*) = [] (void* Param) -> void * {
		typedef void (*EntryType)(void);
		(EntryType(Param))();
		return nullptr;
	};

	pthread_t ThreadHandle;
	if (pthread_create(&ThreadHandle, nullptr, PthreadThunk, reinterpret_cast<void *>(Entry)) != 0)
	{
		return 0;
	}
	return static_cast<UPTRINT>(ThreadHandle);
}

////////////////////////////////////////////////////////////////////////////////
void ThreadSleep(uint32 Milliseconds)
{
	usleep(Milliseconds * 1000U);
}

////////////////////////////////////////////////////////////////////////////////
void ThreadJoin(UPTRINT Handle)
{
	pthread_join(static_cast<pthread_t>(Handle), nullptr);
}

////////////////////////////////////////////////////////////////////////////////
void ThreadDestroy(UPTRINT Handle)
{
	// no-op
}



////////////////////////////////////////////////////////////////////////////////
uint64 TimeGetFrequency()
{
	return 10000000ull;
}

////////////////////////////////////////////////////////////////////////////////
TRACELOG_API uint64 TimeGetTimestamp()
{
	// should stay in sync with FPlatformTime::Cycles64() or the timeline will be broken!
	struct timespec TimeSpec;
	clock_gettime(CLOCK_MONOTONIC, &TimeSpec);
	return static_cast<uint64>(static_cast<uint64>(TimeSpec.tv_sec) * 10000000ULL + static_cast<uint64>(TimeSpec.tv_nsec) / 100ULL);
}

////////////////////////////////////////////////////////////////////////////////
int TcpSocketNative(UPTRINT Socket)
{
	return int(Socket-1);
}

////////////////////////////////////////////////////////////////////////////////
static bool TcpSocketSetNonBlockingInternal(int Socket, bool bNonBlocking)
{
	int Flags = fcntl(Socket, F_GETFL, 0);
	if (Flags == -1)
	{
		return false;
	}

	Flags = bNonBlocking ? (Flags|O_NONBLOCK) : (Flags & ~O_NONBLOCK);
	return fcntl(Socket, F_SETFL, Flags) >= 0;
}

////////////////////////////////////////////////////////////////////////////////
bool TcpSocketSetNonBlocking(UPTRINT Socket, bool bNonBlocking)
{
	int Inner = TcpSocketNative(Socket);
	return TcpSocketSetNonBlockingInternal(Inner, bNonBlocking);
}

////////////////////////////////////////////////////////////////////////////////
UPTRINT TcpSocketConnect(const ANSICHAR* Host, uint16 Port)
{
	struct FAddrInfoPtr
	{
		~FAddrInfoPtr()	
		{
			if (Value != nullptr)
			{
				freeaddrinfo(Value);
			}
		}
		addrinfo*	operator -> ()	{ return Value; }
		addrinfo**	operator & ()	{ return &Value; }
		addrinfo*	Value;
	};

	FAddrInfoPtr Info;
	addrinfo Hints = {};
	Hints.ai_family = AF_INET;
	Hints.ai_socktype = SOCK_STREAM;
	Hints.ai_protocol = IPPROTO_TCP;
	if (getaddrinfo(Host, nullptr, &Hints, &Info))
	{
		Info.Value = nullptr;
		return 0;
	}

	if (&Info == nullptr)
	{
		return 0;
	}

	auto* SockAddr = (sockaddr_in*)Info->ai_addr;
	SockAddr->sin_port = htons(Port);

	int Socket = socket(AF_INET, SOCK_STREAM, IPPROTO_TCP);
	if (Socket < 0)
	{
		return 0;
	}

	int Result = connect(Socket, Info->ai_addr, int(Info->ai_addrlen));
	if (Result < 0)
	{
		close(Socket);
		return 0;
	}

	if (!TcpSocketSetNonBlockingInternal(Socket, false))
	{
		close(Socket);
		return 0;
	}

	return UPTRINT(Socket + 1);
}

////////////////////////////////////////////////////////////////////////////////
UPTRINT TcpSocketListen(uint16 Port)
{
	int Socket = socket(AF_INET, SOCK_STREAM, IPPROTO_TCP);
	if (Socket < 0)
	{
		return 0;
	}

	sockaddr_in SockAddr;
	SockAddr.sin_family = AF_INET;
	SockAddr.sin_addr.s_addr = 0;
	SockAddr.sin_port = htons(Port);
	int Result = bind(Socket, reinterpret_cast<sockaddr*>(&SockAddr), sizeof(SockAddr));
	if (Result < 0)
	{
		close(Socket);
		return 0;
	}

	Result = listen(Socket, 1);
	if (Result < 0)
	{
		close(Socket);
		return 0;
	}

	if (!TcpSocketSetNonBlockingInternal(Socket, true))
	{
		close(Socket);
		return 0;
	}

	return UPTRINT(Socket + 1);
}

////////////////////////////////////////////////////////////////////////////////
int32 TcpSocketAccept(UPTRINT Socket, UPTRINT& Out, bool bAsNonBlocking /* = false */)
{
	int Inner = TcpSocketNative(Socket);

	Inner = accept(Inner, nullptr, nullptr);
	if (Inner < 0)
	{
		return (errno == EAGAIN || errno == EWOULDBLOCK) - 1; // 0 if would block else -1
	}

	if (!TcpSocketSetNonBlockingInternal(Inner, bAsNonBlocking))
	{
		close(Inner);
		return 0;
	}

	Out = UPTRINT(Inner + 1);
	return 1;
}

////////////////////////////////////////////////////////////////////////////////
bool TcpSocketHasData(UPTRINT Socket)
{
	int Inner = TcpSocketNative(Socket);
	fd_set FdSet;
	FD_ZERO(&FdSet);
	FD_SET(Inner, &FdSet);
	timeval TimeVal = {};
	return (select(Inner + 1, &FdSet, nullptr, nullptr, &TimeVal) != 0);
}

////////////////////////////////////////////////////////////////////////////////
bool TcpSocketPeerName(UPTRINT Socket, char* Name, uint32 NameSize)
{
	sockaddr_storage Addr;
	socklen_t AddrLen = sizeof(Addr);

	getpeername(int(Socket-1), (sockaddr*) &Addr, &AddrLen);
	if (Addr.ss_family != AF_INET)
	{
		return false;
	}

	sockaddr_in* Ip4Addr = (sockaddr_in*) &Addr;
	inet_ntop(AF_INET, &Ip4Addr->sin_addr, Name, NameSize);

	return true;
}

////////////////////////////////////////////////////////////////////////////////
bool IoWrite(UPTRINT Handle, const void* Data, uint32 Size)
{
	int Inner = int(Handle) - 1;
	return write(Inner, Data, Size) == Size;
}

////////////////////////////////////////////////////////////////////////////////
int32 IoRead(UPTRINT Handle, void* Data, uint32 Size)
{
	int Inner = int(Handle) - 1;
	return read(Inner, Data, Size);
}

////////////////////////////////////////////////////////////////////////////////
void IoClose(UPTRINT Handle)
{
	int Inner = int(Handle) - 1;
	close(Inner);
}



////////////////////////////////////////////////////////////////////////////////
UPTRINT FileOpen(const ANSICHAR* Path)
{
	int Flags = O_CREAT|O_WRONLY|O_TRUNC;
	int Mode = S_IRUSR|S_IWUSR|S_IRGRP|S_IWGRP|S_IROTH|S_IWOTH;

	int Out = open(Path, Flags, Mode);
	if (Out < 0)
	{
		return 0;
	}

	return UPTRINT(Out + 1);
}

////////////////////////////////////////////////////////////////////////////////
int32 GetLastErrorCode()
{
	return errno;
}

////////////////////////////////////////////////////////////////////////////////
bool GetErrorMessage(char* OutBuffer, uint32 BufferSize, int32 ErrorCode)
{
	const char* ErrorMessage = std::strerror(ErrorCode);
	const bool bResult = strncpy(OutBuffer, ErrorMessage, BufferSize) != 0;
	OutBuffer[BufferSize-1] = 0;
	return bResult;
}

} // namespace Private
} // namespace Trace
} // namespace UE

#endif // TRACE_PRIVATE_MINIMAL_ENABLED
