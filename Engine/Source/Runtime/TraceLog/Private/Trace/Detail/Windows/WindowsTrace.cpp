// Copyright Epic Games, Inc. All Rights Reserved.

#include "Trace/Config.h"

#if TRACE_PRIVATE_MINIMAL_ENABLED && PLATFORM_WINDOWS

#include "Windows/AllowWindowsPlatformTypes.h"
#	define _WINSOCK_DEPRECATED_NO_WARNINGS
#	include <winsock2.h>
#	include <ws2tcpip.h>
#	pragma comment(lib, "ws2_32.lib")
#include "Windows/HideWindowsPlatformTypes.h"

#pragma warning(push)
#pragma warning(disable : 6031) // WSAStartup() return ignore  - we're error tolerant

namespace UE {
namespace Trace {
namespace Private {

struct FSetLastErrorScope
{
	FSetLastErrorScope(DWORD InError)
		: Error(InError){}
	~FSetLastErrorScope()
	{
		SetLastError(Error);
	}

private:
	DWORD Error;
};

////////////////////////////////////////////////////////////////////////////////
UPTRINT ThreadCreate(const ANSICHAR*, void (*Entry)())
{
	DWORD (WINAPI *WinApiThunk)(void*) = [] (void* Param) -> DWORD
	{
		typedef void (*EntryType)(void);
		(EntryType(Param))();
		return 0;
	};

	HANDLE Handle = CreateThread(nullptr, 0, WinApiThunk, (void*)Entry, 0, nullptr);
	return UPTRINT(Handle);
}

////////////////////////////////////////////////////////////////////////////////
void ThreadSleep(uint32 Milliseconds)
{
	Sleep(Milliseconds);
}

////////////////////////////////////////////////////////////////////////////////
void ThreadJoin(UPTRINT Handle)
{
	WaitForSingleObject(HANDLE(Handle), INFINITE);
}

////////////////////////////////////////////////////////////////////////////////
void ThreadDestroy(UPTRINT Handle)
{
	CloseHandle(HANDLE(Handle));
}



////////////////////////////////////////////////////////////////////////////////
uint64 TimeGetFrequency()
{
	LARGE_INTEGER Value;
	QueryPerformanceFrequency(&Value);
	return Value.QuadPart;
}

////////////////////////////////////////////////////////////////////////////////
TRACELOG_API uint64 TimeGetTimestamp()
{
	LARGE_INTEGER Value;
	QueryPerformanceCounter(&Value);
	return Value.QuadPart;
}



////////////////////////////////////////////////////////////////////////////////
static void TcpSocketInitialize()
{
	WSADATA WsaData;
	WSAStartup(MAKEWORD(2, 2), &WsaData);
}

////////////////////////////////////////////////////////////////////////////////
int TcpSocketNative(UPTRINT Socket)
{
	return int(Socket-1);
}

////////////////////////////////////////////////////////////////////////////////
static bool TcpSocketSetNonBlockingInternal(SOCKET Socket, bool bNonBlocking)
{
	unsigned long NonBlockingMode = !!bNonBlocking;
	return ioctlsocket(Socket, FIONBIO, &NonBlockingMode) != SOCKET_ERROR;
}

////////////////////////////////////////////////////////////////////////////////
bool TcpSocketSetNonBlocking(UPTRINT Socket, bool bNonBlocking)
{
	SOCKET Inner = (SOCKET) TcpSocketNative(Socket);
	return TcpSocketSetNonBlockingInternal(Inner, bNonBlocking);
}

////////////////////////////////////////////////////////////////////////////////
UPTRINT TcpSocketConnect(const ANSICHAR* Host, uint16 Port)
{
	TcpSocketInitialize();

	struct FAddrInfoPtr
	{
					~FAddrInfoPtr()	{ freeaddrinfo(Value); }
		addrinfo*	operator -> ()	{ return Value; }
		addrinfo**	operator & ()	{ return &Value; }
		addrinfo*	Value;
	};

	FAddrInfoPtr Info;
	addrinfo Hints = {};
	Hints.ai_family = AF_INET;
	Hints.ai_socktype = SOCK_STREAM;
	Hints.ai_protocol = IPPROTO_TCP;
	if (getaddrinfo(Host, nullptr, &Hints, &Info))
	{
		return 0;
	}

	if (&Info == nullptr)
	{
		return 0;
	}

	auto* SockAddr = (sockaddr_in*)Info->ai_addr;
	SockAddr->sin_port = htons(Port);

	// socket() will create a socket with overlapped IO support which we don't
	// want as it complicates sharing Io*() API with FileOpen(). So we use
	// WSASocket instead which affords us more control over socket properties
	SOCKET Socket = WSASocket(AF_INET, SOCK_STREAM, IPPROTO_TCP, nullptr, 0, WSA_FLAG_NO_HANDLE_INHERIT);
	if (Socket == INVALID_SOCKET)
	{
		return 0;
	}

	int Result = connect(Socket, Info->ai_addr, int(Info->ai_addrlen));
	if (Result == SOCKET_ERROR)
	{
		FSetLastErrorScope _(GetLastError());
		closesocket(Socket);
		return 0;
	}

	if (!TcpSocketSetNonBlockingInternal(Socket, 0))
	{
		FSetLastErrorScope _(GetLastError());
		closesocket(Socket);
		return 0;
	}

	return UPTRINT(Socket) + 1;
}

////////////////////////////////////////////////////////////////////////////////
UPTRINT TcpSocketListen(uint16 Port)
{
	TcpSocketInitialize();

	// See TcpSocketConnect() for why WSASocket() is used here.
	SOCKET Socket = WSASocket(AF_INET, SOCK_STREAM, IPPROTO_TCP, nullptr, 0, WSA_FLAG_NO_HANDLE_INHERIT);
	if (Socket == INVALID_SOCKET)
	{
		return 0;
	}

	sockaddr_in SockAddr;
	SockAddr.sin_family = AF_INET;
	SockAddr.sin_addr.s_addr = 0;
	SockAddr.sin_port = htons(Port);
	int Result = bind(Socket, (SOCKADDR*)&SockAddr, sizeof(SockAddr));
	if (Result == INVALID_SOCKET)
	{
		FSetLastErrorScope _(GetLastError());
		closesocket(Socket);
		return 0;
	}

	Result = listen(Socket, 1);
	if (Result == INVALID_SOCKET)
	{
		FSetLastErrorScope _(GetLastError());
		closesocket(Socket);
		return 0;
	}

	if (!TcpSocketSetNonBlockingInternal(Socket, 1))
	{
		FSetLastErrorScope _(GetLastError());
		closesocket(Socket);
		return 0;
	}

	return UPTRINT(Socket) + 1;
}

////////////////////////////////////////////////////////////////////////////////
int32 TcpSocketAccept(UPTRINT Socket, UPTRINT& Out, bool bAsNonBlocking /*= false*/)
{
	SOCKET Inner = (SOCKET) TcpSocketNative(Socket);

	Inner = accept(Inner, nullptr, nullptr);
	if (Inner == INVALID_SOCKET)
	{
		return (WSAGetLastError() == WSAEWOULDBLOCK) - 1; // 0 if would block else -1
	}

	if (!TcpSocketSetNonBlockingInternal(Inner, bAsNonBlocking))
	{
		FSetLastErrorScope _(GetLastError());
		closesocket(Inner);
		return 0;
	}

	Out = UPTRINT(Inner) + 1;
	return 1;
}

////////////////////////////////////////////////////////////////////////////////
bool TcpSocketHasData(UPTRINT Socket)
{
	SOCKET Inner = (SOCKET) TcpSocketNative(Socket);
	fd_set FdSet = { 1, { Inner }, };
	TIMEVAL TimeVal = {};
	return (select(0, &FdSet, nullptr, nullptr, &TimeVal) != 0);
}


////////////////////////////////////////////////////////////////////////////////
bool TcpSocketPeerName(UPTRINT Socket, char* Name, uint32 NameSize)
{
	sockaddr_storage Addr;
	socklen_t AddrLen = sizeof(Addr);

	getpeername(int(Socket-1), (sockaddr*) &Addr, &AddrLen);
	if (Addr.ss_family != AF_INET)
	{
		return false;
	}

	sockaddr_in* Ip4Addr = (sockaddr_in*) &Addr;
	inet_ntop(AF_INET, &Ip4Addr->sin_addr, Name, NameSize);

	return true;
}

////////////////////////////////////////////////////////////////////////////////
bool IoWrite(UPTRINT Handle, const void* Data, uint32 Size)
{
	HANDLE Inner = HANDLE(Handle - 1);

	DWORD BytesWritten = 0;
	if (!WriteFile(Inner, (const char*)Data, Size, &BytesWritten, nullptr))
	{
		return false;
	}

	return (BytesWritten == Size);
}

////////////////////////////////////////////////////////////////////////////////
int32 IoRead(UPTRINT Handle, void* Data, uint32 Size)
{
	HANDLE Inner = HANDLE(Handle - 1);

	DWORD BytesRead = 0;
	if (!ReadFile(Inner, (char*)Data, Size, &BytesRead, nullptr))
	{
		return -1;
	}

	return BytesRead;
}

////////////////////////////////////////////////////////////////////////////////
void IoClose(UPTRINT Handle)
{
	HANDLE Inner = HANDLE(Handle - 1);
	CloseHandle(Inner);
}



////////////////////////////////////////////////////////////////////////////////
UPTRINT FileOpen(const ANSICHAR* Path)
{
	DWORD Access = GENERIC_WRITE;
	DWORD Share = FILE_SHARE_READ;
	DWORD Disposition = CREATE_ALWAYS;
	DWORD Flags = FILE_ATTRIBUTE_NORMAL;
	HANDLE Out = CreateFileA(Path, Access, Share, nullptr, Disposition, Flags, nullptr);
	if (Out == INVALID_HANDLE_VALUE)
	{
		return 0;
	}

	return UPTRINT(Out) + 1;
}

	
////////////////////////////////////////////////////////////////////////////////
int32 GetLastErrorCode()
{
	return ::GetLastError();
}

////////////////////////////////////////////////////////////////////////////////
bool GetErrorMessage(char* OutBuffer, uint32 BufferSize, int32 ErrorCode)
{
	return FormatMessageA(FORMAT_MESSAGE_FROM_SYSTEM, NULL, ErrorCode, 0, OutBuffer, BufferSize, NULL) != 0;
}
	
} // namespace Private
} // namespace Trace
} // namespace UE

#pragma warning(pop)

#endif // TRACE_PRIVATE_MINIMAL_ENABLED
