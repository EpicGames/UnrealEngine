// Copyright Epic Games, Inc. All Rights Reserved.

#include "Trace/Detail/EventNode.h"

#if TRACE_PRIVATE_MINIMAL_ENABLED

#include "Trace/Detail/Atomic.h"
#include "Trace/Detail/LogScope.inl"

namespace UE {
namespace Trace {
namespace Private {

////////////////////////////////////////////////////////////////////////////////
void					Writer_InternalInitialize();
FEventNode* volatile	GNewEventList; // = nullptr;
FEventNode*				GEventListHead;// = nullptr;
FEventNode*				GEventListTail;// = nullptr;



////////////////////////////////////////////////////////////////////////////////
const FEventNode* FEventNode::FIter::GetNext()
{
	auto* Ret = (FEventNode*)Inner;
	if (Ret != nullptr)
	{
		Inner = Ret->Next;

		if (Inner == nullptr)
		{
			GEventListTail = Ret;
		}
	}
	return Ret;
}

////////////////////////////////////////////////////////////////////////////////
FEventNode::FIter FEventNode::Read()
{
	// This is the current event list as of the last ReadNew() call
	if (GEventListHead)
	{
		return { GEventListHead };
	}

	// This is the current event list if ReadNew() has never been called
	if (GNewEventList)
	{
		return { GNewEventList };
	}

	return {};
}

////////////////////////////////////////////////////////////////////////////////
FEventNode::FIter FEventNode::ReadNew()
{
	FEventNode* EventList = AtomicExchangeAcquire(&GNewEventList, (FEventNode*)nullptr);
	if (EventList == nullptr)
	{
		return {};
	}

	if (GEventListHead == nullptr)
	{
		GEventListHead = EventList;
	}
	else
	{
		GEventListTail->Next = EventList;
	}

	return { EventList };
}

////////////////////////////////////////////////////////////////////////////////
uint32 FEventNode::Initialize(const FEventInfo* InInfo)
{
	if (Uid != 0)
	{
		return Uid;
	}

	// If we're initializing an event we're about to trace it
	Writer_InternalInitialize();

	// Assign a unique ID for this event
	static uint32 volatile EventUidCounter; // = 0;
	uint32 NewUid = AtomicAddRelaxed(&EventUidCounter, 1u) + EKnownEventUids::User;
	if (NewUid >= uint32(EKnownEventUids::Max))
	{
		return Uid = EKnownEventUids::Invalid;
	}

	// Calculate Uid's flags and pack it.
	uint32 UidFlags = 0;
	if (NewUid >= (1 << (8 - EKnownEventUids::_UidShift)))
	{
		UidFlags |= EKnownEventUids::Flag_TwoByteUid;
	}

	NewUid <<= EKnownEventUids::_UidShift;
	NewUid |= UidFlags;

	Info = InInfo;
	Uid = uint16(NewUid);

	// Make this new event instance visible.
	for (;; PlatformYield())
	{
		Next = AtomicLoadRelaxed(&GNewEventList);
		if (AtomicCompareExchangeRelease(&GNewEventList, this, Next))
		{
			break;
		}
	}

	return Uid;
}

////////////////////////////////////////////////////////////////////////////////
void FEventNode::Describe() const
{
	const FLiteralName& LoggerName = Info->LoggerName;
	const FLiteralName& EventName = Info->EventName;
	
	// Definition events adds an extra field containing the definition id to the end.
	const uint32 DefinitionIdFieldIdx = Info->FieldCount - ((Info->Flags & FEventInfo::DefinitionBits) ? 1 : 0);

	// Calculate the number of fields and size of name data.
	uint32 NamesSize = LoggerName.Length + EventName.Length;
	for (uint32 i = 0; i < Info->FieldCount; ++i)
	{
		NamesSize += Info->Fields[i].NameSize;
	}

	// Allocate the "NewEvent" event in the log stream.
	uint32 EventSize = sizeof(FNewEventEvent);
	EventSize += sizeof(FNewEventField) * Info->FieldCount;
	EventSize += NamesSize;
	EventSize = (EventSize + 1) & ~1; // align to 2 to keep UBSAN happy

	FLogScope LogScope = FLogScope::EnterImpl<FEventInfo::Flag_NoSync>(0, EventSize + sizeof(uint16));
	auto* Ptr = (uint16*)(LogScope.GetPointer());
	Ptr[-1] = EKnownEventUids::NewEvent; // Make event look like an important one. Ideally they are sent
	Ptr[ 0] = uint16(EventSize);		 // as important and not Writer_DescribeEvents()'s redirected buf.

	// Write event's main properties.
	auto& Event = *(FNewEventEvent*)(Ptr + 1);
	Event.EventUid = uint16(Uid) >> EKnownEventUids::_UidShift;
	Event.LoggerNameSize = LoggerName.Length;
	Event.EventNameSize = EventName.Length;
	Event.Flags = 0;

	const uint32 Flags = Info->Flags;
	if (Flags & FEventInfo::Flag_Important)		Event.Flags |= uint8(EEventFlags::Important);
	if (Flags & FEventInfo::Flag_MaybeHasAux)	Event.Flags |= uint8(EEventFlags::MaybeHasAux);
	if (Flags & FEventInfo::Flag_NoSync)		Event.Flags |= uint8(EEventFlags::NoSync);
	if (Flags & FEventInfo::DefinitionBits)		Event.Flags |= uint8(EEventFlags::Definition);

	// Write details about event's fields
	Event.FieldCount = uint8(Info->FieldCount);
	for (uint32 i = 0; i < Info->FieldCount; ++i)
	{
		const FFieldDesc& Field = Info->Fields[i];
		FNewEventField& Out = Event.Fields[i];
		if (i == DefinitionIdFieldIdx)
		{
			Out.FieldType = EFieldFamily::DefinitionId;
			Out.DefinitionId.Offset = Field.ValueOffset;
			Out.DefinitionId.TypeInfo = Field.TypeInfo;
		}
		else if (Field.Reference != 0)
		{
			Out.FieldType = EFieldFamily::Reference;
			// todo: What if the referenced type has not been initialized yet?
			Out.Reference.Offset = Field.ValueOffset;
			Out.Reference.TypeInfo = Field.TypeInfo;
			Out.Reference.NameSize = Field.NameSize;
			Out.Reference.RefUid = uint16(Field.Reference) >> EKnownEventUids::_UidShift;
		}
		else
		{
			Out.FieldType = EFieldFamily::Regular;
			Out.Regular.Offset = Field.ValueOffset;
			Out.Regular.Size = Field.ValueSize;
			Out.Regular.TypeInfo = Field.TypeInfo;
			Out.Regular.NameSize = Field.NameSize;
		}
	}

	// Write names
	uint8* Cursor = (uint8*)(Event.Fields + Info->FieldCount);
	auto WriteName = [&Cursor] (const ANSICHAR* Data, uint32 Size)
	{
		memcpy(Cursor, Data, Size);
		Cursor += Size;
	};

	WriteName(LoggerName.Ptr, LoggerName.Length);
	WriteName(EventName.Ptr, EventName.Length);
	for (uint32 i = 0; i < Info->FieldCount; ++i)
	{
		const FFieldDesc& Field = Info->Fields[i];
		WriteName(Field.Name, Field.NameSize);
	}

	LogScope.Commit();
}

////////////////////////////////////////////////////////////////////////////////
void FEventNode::OnConnect()
{
	// Re-add known events back as new events so that they get described again
	if (GEventListHead == nullptr)
	{
		return;
	}

	GEventListTail->Next = AtomicExchangeAcquire(&GNewEventList, GEventListHead);

	GEventListHead = GEventListTail = nullptr;
}

} // namespace Private
} // namespace Trace
} // namespace UE

#endif // TRACE_PRIVATE_MINIMAL_ENABLED

