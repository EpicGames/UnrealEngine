// Copyright Epic Games, Inc. All Rights Reserved.

#include "Trace/Detail/Field.h" // :(

#if TRACE_PRIVATE_MINIMAL_ENABLED

#include "Trace/Detail/Writer.inl"

namespace UE {
namespace Trace {
namespace Private {

////////////////////////////////////////////////////////////////////////////////
template <typename CallbackType>
static void Field_WriteAuxData(uint32 Index, int32 Size, CallbackType&& Callback)
{
	static_assert(
		// "1+" is so we've at least space to write one byte of payload
		sizeof(Private::FWriteBuffer::Overflow) >= 1 + sizeof(FAuxHeader) + sizeof(uint8 /*AuxDataTerminal*/),
		"FWriteBuffer::Overflow is not large enough"
	);

	// Early-out if there would be nothing to write
	if (Size == 0)
	{
		return;
	}

	FWriteBuffer* Buffer = Writer_GetBuffer();

	// We are writing fields of an event. The event writing will move Commited
	// along. However, if we're writing a second aux field where the first fetched
	// a new buffer then it is a different buffer to the one the event's written
	// into, thus it is us that needs to update Committed.
	bool bCommit = (Buffer->ReadCursor() == Buffer->Committed);

	// Do we have enough space to write at least [header][overflow-1] bytes of payload?
	int32 Remaining = int32(ptrdiff_t((uint8*)Buffer - Buffer->ReadCursor()));
	auto NextBuffer = [&Buffer, &Remaining, &bCommit] ()
	{
		if (bCommit)
		{
			AtomicStoreRelease(&(uint8* volatile&)(Buffer->Committed), Buffer->ReadCursor());
		}
		bCommit = true;

		Buffer = Writer_NextBuffer(Buffer);
		Remaining = int32(ptrdiff_t((uint8*)Buffer - Buffer->ReadCursor()));
	};

	if (Remaining <= 0)
	{
		NextBuffer();
	}

	while (true)
	{
		// Buffers have a small overflow which we can use. It means we can write
		// some elements unconditionally. What remains is free for payload data.
		Remaining += sizeof(FWriteBuffer::Overflow);
		Remaining -= 1;						// for the aux-terminal
		Remaining -= sizeof(FAuxHeader);	// header also assume to always fit
		int32 SegmentSize = (Remaining < Size) ? Remaining : Size;

		// Write header
		uint32 Pack = SegmentSize << FAuxHeader::SizeShift;
		Pack |= Index << FAuxHeader::FieldShift;
		memcpy(Buffer->ReadCursor(), &Pack, sizeof(uint32)); /* FAuxHeader::Pack */
		*Buffer->ReadCursor() = uint8(EKnownEventUids::AuxData) << EKnownEventUids::_UidShift; /* FAuxHeader::Uid */
		Buffer->MoveCursor(sizeof(FAuxHeader));

		// Write payload data
		Callback(Buffer->ReadCursor(), SegmentSize);
		Buffer->MoveCursor((uint16)SegmentSize);

		// Bounds check
		Size -= SegmentSize;
		if (Size <= 0)
		{
			break;
		}

		NextBuffer();
	}

	if (bCommit)
	{
		AtomicStoreRelease(&(uint8* volatile&)(Buffer->Committed), Buffer->ReadCursor());
	}
}

////////////////////////////////////////////////////////////////////////////////
void Field_WriteAuxData(uint32 Index, const uint8* Data, int32 Size)
{
	auto MemcpyLambda = [&Data] (uint8* Cursor, int32 NumBytes)
	{
		memcpy(Cursor, Data, NumBytes);
		Data += NumBytes;
	};
	return Field_WriteAuxData(Index, Size, MemcpyLambda);
}

////////////////////////////////////////////////////////////////////////////////
void Field_WriteStringAnsi(uint32 Index, const WIDECHAR* String, int32 Length)
{
	int32 Size = Length;
	Size &= (FAuxHeader::SizeLimit - 1);

	auto WriteLambda = [&String] (uint8* Cursor, int32 NumBytes)
	{
		for (int32 i = 0; i < NumBytes; ++i)
		{
			*Cursor = uint8(*String & 0x7f);
			Cursor++;
			String++;
		}
	};

	return Field_WriteAuxData(Index, Size, WriteLambda);
}

////////////////////////////////////////////////////////////////////////////////
void Field_WriteStringAnsi(uint32 Index, const ANSICHAR* String, int32 Length)
{
	int32 Size = Length * sizeof(String[0]);
	Size &= (FAuxHeader::SizeLimit - 1); // a very crude "clamp"
	return Field_WriteAuxData(Index, (const uint8*)String, Size);
}

////////////////////////////////////////////////////////////////////////////////
void Field_WriteStringWide(uint32 Index, const WIDECHAR* String, int32 Length)
{
	int32 Size = Length * sizeof(String[0]);
	Size &= (FAuxHeader::SizeLimit - 1); // (see above)
	return Field_WriteAuxData(Index, (const uint8*)String, Size);
}

} // namespace Private
} // namespace Trace
} // namespace UE

#endif // TRACE_PRIVATE_MINIMAL_ENABLED
