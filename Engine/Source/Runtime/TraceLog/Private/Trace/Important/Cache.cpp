// Copyright Epic Games, Inc. All Rights Reserved.

#include "Trace/Config.h"

#if TRACE_PRIVATE_MINIMAL_ENABLED && TRACE_PRIVATE_ALLOW_IMPORTANTS

#include "Trace/Platform.h"
#include "Trace/Detail/Atomic.h"
#include "Trace/Detail/Protocol.h"
#include "Trace/Detail/Transport.h"
#include "Trace/Trace.h"

#include <memory.h>

namespace UE {
namespace Trace {
namespace Private {

////////////////////////////////////////////////////////////////////////////////
uint32	GetEncodeMaxSize(uint32);
int32	EncodeInstr(const void*, int32, void*, int32);
void*	Writer_MemoryAllocate(SIZE_T, uint32);
void	Writer_MemoryFree(void*, uint32);
void	Writer_SendDataRaw(const void*, uint32);
void	Writer_SendData(uint32, uint8* __restrict, uint32);

////////////////////////////////////////////////////////////////////////////////
struct alignas(16) FCacheBuffer
{
	union
	{
		FCacheBuffer*	Next;
		FCacheBuffer**	TailNext;
	};
	uint32				Size;
	uint32				Remaining;
	uint32				_Unused[3];
	uint32				Underflow; // For packet header
	uint8				Data[];
};



////////////////////////////////////////////////////////////////////////////////
static const uint32		GCacheBufferSize	= 64 << 10;
static const uint32		GCacheCollectorSize	= 1 << 10;
static FCacheBuffer*	GCacheCollector;	// = nullptr;
static FCacheBuffer*	GCacheActiveBuffer;	// = nullptr;
static FCacheBuffer*	GCacheHeadBuffer;	// = nullptr;
extern FStatistics		GTraceStatistics;

////////////////////////////////////////////////////////////////////////////////
static FCacheBuffer* Writer_CacheCreateBuffer(uint32 Size)
{
	const uint32 CacheBufferSize = sizeof(FCacheBuffer) + Size;
	void* Block = Writer_MemoryAllocate(CacheBufferSize, alignof(FCacheBuffer));
#if TRACE_PRIVATE_STATISTICS
	AtomicAddRelaxed(&GTraceStatistics.CacheAllocated, CacheBufferSize);
#endif
	auto* Buffer = (FCacheBuffer*)Block;
	Buffer->Size = Size;
	Buffer->Remaining = Buffer->Size;
	Buffer->Next = nullptr;
	return Buffer;
}

////////////////////////////////////////////////////////////////////////////////
static void Writer_CacheCommit(const FCacheBuffer* Collector)
{
	FProfilerScope _(__func__);

	// Check there's enough size to compress Data into.
	uint32 InputSize = uint32(Collector->Size - Collector->Remaining);
	uint32 EncodeMaxSize = GetEncodeMaxSize(InputSize);
	if (EncodeMaxSize + sizeof(FTidPacketEncoded) > GCacheActiveBuffer->Remaining)
	{
#if TRACE_PRIVATE_STATISTICS
		AtomicAddRelaxed(&GTraceStatistics.CacheWaste, GCacheActiveBuffer->Remaining);
#endif

		// Retire active buffer
		*(GCacheActiveBuffer->TailNext) = GCacheActiveBuffer;
		GCacheActiveBuffer->TailNext = nullptr;

		// Fire up a new active buffer
		FCacheBuffer* NewBuffer = Writer_CacheCreateBuffer(GCacheBufferSize);
		NewBuffer->TailNext = &(GCacheActiveBuffer->Next);
		GCacheActiveBuffer = NewBuffer;
	}

	const uint32 ActiveBufferOffset = GCacheActiveBuffer->Size - GCacheActiveBuffer->Remaining;
	uint32 PacketTotalSize = 0;
	auto* Packet = (FTidPacketEncoded*)(GCacheActiveBuffer->Data + ActiveBufferOffset);
	const int32 CompressedSize = EncodeInstr(Collector->Data, InputSize, Packet->Data, EncodeMaxSize);
	if (CompressedSize > 0)
	{
		Packet->PacketSize = uint16(CompressedSize + sizeof(FTidPacketEncoded));
		Packet->ThreadId = FTidPacketBase::EncodedMarker | uint16(ETransportTid::Importants);
		Packet->DecodedSize = uint16(InputSize);

		PacketTotalSize = sizeof(FTidPacketEncoded) + CompressedSize;
	}
	else
	{
		// Compression failed. We can reuse the same buffer for uncompressed packet
		// since the worst case size for compression is (size + compressionoverhead)
		// and FTidPacket < FTidPackedEncoded.
		auto* UncompressedPacket = (FTidPacket*)(GCacheActiveBuffer->Data + ActiveBufferOffset);
		UncompressedPacket->ThreadId = uint16(ETransportTid::Importants);
#if UE_TRACE_PACKET_VERIFICATION
		Packet->ThreadId |= FTidPacketBase::Verification;
#endif
		UncompressedPacket->PacketSize = uint16(InputSize) + sizeof(FTidPacket);
		::memcpy(UncompressedPacket->Data, Collector->Data, InputSize);

		PacketTotalSize = sizeof(FTidPacket) + InputSize;
	}

	GCacheActiveBuffer->Remaining -= PacketTotalSize;

#if TRACE_PRIVATE_STATISTICS
	AtomicAddRelaxed(&GTraceStatistics.CacheUsed, PacketTotalSize);
#endif
}

////////////////////////////////////////////////////////////////////////////////
void Writer_CacheData(uint8* Data, uint32 Size)
{
	FProfilerScope _(__func__);

	Writer_SendData(ETransportTid::Importants, Data, Size);

	if (GCacheCollector == nullptr)
	{
		return;
	}

	while (true)
	{
		uint32 StepSize = (Size < GCacheCollector->Remaining) ? Size : GCacheCollector->Remaining;

		uint32 Used = GCacheCollector->Size - GCacheCollector->Remaining;
		memcpy(GCacheCollector->Data + Used, Data, StepSize);

		GCacheCollector->Remaining -= StepSize;
		if (GCacheCollector->Remaining == 0)
		{
			Writer_CacheCommit(GCacheCollector);
			GCacheCollector->Remaining = GCacheCollector->Size;
		}

		Size -= StepSize;
		if (Size == 0)
		{
			break;
		}

		Data += StepSize;
	}
}

////////////////////////////////////////////////////////////////////////////////
void Writer_CacheOnConnect()
{
	FProfilerScope _(__func__);

	if (GCacheCollector == nullptr)
	{
		return;
	}

	for (FCacheBuffer* Buffer = GCacheHeadBuffer; Buffer != nullptr; Buffer = Buffer->Next)
	{
		uint32 Used = Buffer->Size - Buffer->Remaining;
		Writer_SendDataRaw(Buffer->Data, Used);
	}

	if (uint32 Used = GCacheActiveBuffer->Size - GCacheActiveBuffer->Remaining)
	{
		Writer_SendDataRaw(GCacheActiveBuffer->Data, Used);
	}

	if (uint32 Used = GCacheCollector->Size - GCacheCollector->Remaining)
	{
		Writer_SendData(ETransportTid::Importants, GCacheCollector->Data, Used);
	}
}

////////////////////////////////////////////////////////////////////////////////
void Writer_InitializeCache()
{
	GCacheCollector = Writer_CacheCreateBuffer(GCacheCollectorSize);

	GCacheActiveBuffer = Writer_CacheCreateBuffer(GCacheBufferSize);
	GCacheActiveBuffer->TailNext = &GCacheHeadBuffer;

	static_assert(ETransport::Active == ETransport::TidPacketSync, "The important cache is transport aware");
}

////////////////////////////////////////////////////////////////////////////////
void Writer_ShutdownCache()
{
#if TRACE_PRIVATE_STATISTICS
	uint32 TotalCacheToSubtract = 0;
#endif
	for (FCacheBuffer* Buffer = GCacheHeadBuffer; Buffer != nullptr;)
	{
		FCacheBuffer* Next = Buffer->Next;
		const uint32 CacheBufferSize = Buffer->Size + sizeof(FCacheBuffer);
		Writer_MemoryFree(Buffer, CacheBufferSize);
#if TRACE_PRIVATE_STATISTICS
		TotalCacheToSubtract += CacheBufferSize;
#endif
		Buffer = Next;
	}

	Writer_MemoryFree(GCacheActiveBuffer, GCacheBufferSize);
	Writer_MemoryFree(GCacheCollector, GCacheCollectorSize);
#if TRACE_PRIVATE_STATISTICS
	AtomicSubRelaxed(&GTraceStatistics.CacheAllocated, TotalCacheToSubtract + GCacheBufferSize + GCacheCollectorSize);
#endif
}

} // namespace Private
} // namespace Trace
} // namespace UE

#endif // TRACE_PRIVATE_MINIMAL_ENABLED && TRACE_PRIVATE_ALLOW_IMPORTANTS
