// Copyright Epic Games, Inc. All Rights Reserved.

#include "Trace/Detail/Important/SharedBuffer.h"

#if TRACE_PRIVATE_MINIMAL_ENABLED && TRACE_PRIVATE_ALLOW_IMPORTANTS

#include "CoreTypes.h"
#include "Misc/ScopeExit.h"
#include "Trace/Detail/Atomic.h"
#include "Trace/Detail/Transport.h"
#include "Trace/Detail/Important/ImportantLogScope.inl"
#include "Trace/Platform.h"

namespace UE {
namespace Trace {
namespace Private {

////////////////////////////////////////////////////////////////////////////////
void*	Writer_MemoryAllocate(SIZE_T, uint32);
void	Writer_MemoryFree(void*, uint32);
void	Writer_CacheData(uint8*, uint32);

////////////////////////////////////////////////////////////////////////////////
static FSharedBuffer	GNullSharedBuffer	= { 0, FSharedBuffer::RefInit };
FSharedBuffer* volatile GSharedBuffer		= &GNullSharedBuffer;
static FSharedBuffer*	GTailBuffer;		// = nullptr
static uint32			GTailPreSent;		// = 0
static const uint32		GBlockSize			= 1 << 10;		// Block size must be a power of two!
extern FStatistics		GTraceStatistics;

////////////////////////////////////////////////////////////////////////////////
static FSharedBuffer* Writer_CreateSharedBuffer(uint32 SizeHint=0)
{
	const uint32 OverheadSize = sizeof(FSharedBuffer) + sizeof(uint32);

	uint32 BlockSize = GBlockSize;
	if (SizeHint + OverheadSize > GBlockSize)
	{
		BlockSize += SizeHint + OverheadSize - GBlockSize;
		BlockSize += GBlockSize - 1;
		BlockSize &= ~(GBlockSize - 1);
	}

	// Clamp max aligned block size to the maximum we can send
	// These huge buffers will be sent uncompressed
	constexpr uint32 MaxBufferSize = UINT16_MAX - sizeof(FTidPacket);
	if (BlockSize > MaxBufferSize + OverheadSize)
	{
		BlockSize = MaxBufferSize + OverheadSize;
	}

	void* Block = Writer_MemoryAllocate(BlockSize, alignof(FSharedBuffer));
	auto* Buffer = (FSharedBuffer*)(UPTRINT(Block) + BlockSize) - 1;
	
#if TRACE_PRIVATE_STATISTICS
	AtomicAddRelaxed(&GTraceStatistics.SharedBufferAllocated, BlockSize);
#endif

	Buffer->Size = uint32(UPTRINT(Buffer) - UPTRINT(Block));
	Buffer->Size -= sizeof(uint32); // to precede event data with a small header when sending.
	Buffer->Cursor = (Buffer->Size << FSharedBuffer::CursorShift) | FSharedBuffer::RefInit;
	Buffer->Next = nullptr;
	Buffer->Final = 0;

	return Buffer;
}

////////////////////////////////////////////////////////////////////////////////
FNextSharedBuffer Writer_NextSharedBuffer(FSharedBuffer* Buffer, int32 RegionStart, int32 NegSizeAndRef)
{
	// Lock free allocation of the next buffer
	FSharedBuffer* NextBuffer;
	while (true)
	{
		bool bBufferOwner = (RegionStart >= 0);
		if (LIKELY(bBufferOwner))
		{
			// Allocate the next buffer ourselves
			uint32 Size = -NegSizeAndRef >> FSharedBuffer::CursorShift;
			NextBuffer = Writer_CreateSharedBuffer(Size);
			Buffer->Next = NextBuffer;
			Buffer->Final = RegionStart >> FSharedBuffer::CursorShift;
			AtomicStoreRelease(&GSharedBuffer, NextBuffer);
		}
		else
		{
			int32 ThrottleRestore = ThreadThrottle();
			ON_SCOPE_EXIT { ThreadUnthrottle(ThrottleRestore); };

			// Another thread is already allocating the next buffer, wait for that to complete
			for (;; PlatformYield())
			{
				NextBuffer = AtomicLoadAcquire(&GSharedBuffer);
				if (NextBuffer != Buffer)
				{
					break;
				}
			}
		}

#if TRACE_PRIVATE_STATISTICS
		AtomicAddRelaxed(&GTraceStatistics.BytesEmitted, uint64(-Buffer->Cursor >> FSharedBuffer::CursorShift));
#endif
		AtomicAddRelease(&(Buffer->Cursor), int32(FSharedBuffer::RefBit));

		// Try and allocate some space in the next buffer.  The next buffer may have insufficient
		// space if another thread used some of the memory we allocated, or if it was a buffer
		// allocated by another thread that's too small to fit our allocation size.
		RegionStart = AtomicAddRelaxed(&(NextBuffer->Cursor), NegSizeAndRef);
		if (LIKELY(RegionStart + NegSizeAndRef >= 0))
		{
			break;
		}

		Buffer = NextBuffer;
	}

	return { NextBuffer, RegionStart };
}

////////////////////////////////////////////////////////////////////////////////
static void Writer_RetireSharedBufferImpl()
{
	// Send any unsent data.
	uint8* Data = (uint8*)GTailBuffer - GTailBuffer->Size + GTailPreSent;
	if (auto SendSize = UPTRINT(GTailBuffer) - UPTRINT(Data) - GTailBuffer->Final)
	{
#if TRACE_PRIVATE_STATISTICS
		AtomicAddRelaxed(&GTraceStatistics.BytesTraced, (uint64)SendSize);
#endif

		Writer_CacheData(Data, uint32(SendSize));
	}

	FSharedBuffer* Temp = GTailBuffer->Next;
	const uint32 AllocSizeBytes = GTailBuffer->Size + sizeof(uint32) + sizeof(FSharedBuffer);
	void* Block = (uint8*)GTailBuffer - GTailBuffer->Size - sizeof(uint32);
	Writer_MemoryFree(Block, AllocSizeBytes);
#if TRACE_PRIVATE_STATISTICS
	AtomicSubRelease(&GTraceStatistics.SharedBufferAllocated, AllocSizeBytes);
#endif
	GTailBuffer = Temp;
	GTailPreSent = 0;
}

////////////////////////////////////////////////////////////////////////////////
static void Writer_RetireSharedBuffer()
{
	// Spin until the buffer's no longer being used.
	for (;; PlatformYield())
	{
		int32 TailCursor = AtomicLoadAcquire(&(GTailBuffer->Cursor));
		if (LIKELY(((TailCursor + 1) & FSharedBuffer::RefInit) == 0))
		{
			break;
		}
	}

	Writer_RetireSharedBufferImpl();
}

////////////////////////////////////////////////////////////////////////////////
void Writer_UpdateSharedBuffers()
{
	FProfilerScope _(__func__);
	int32 ThrottleRestore = -1;
	ON_SCOPE_EXIT {
		if (ThrottleRestore >= 0)
		{
			ThreadUnthrottle(ThrottleRestore);
		}
	};

	auto ApplyThrottle = [&ThrottleRestore] () {
		if (ThrottleRestore < 0)
		{
			ThrottleRestore = ThreadThrottle();
		}
	};

	FSharedBuffer* HeadBuffer = AtomicLoadAcquire(&GSharedBuffer);
	for (;; PlatformYield())
	{
		if (GTailBuffer != HeadBuffer)
		{
			ApplyThrottle();
			Writer_RetireSharedBuffer();
			continue;
		}

		int32 Cursor = AtomicLoadAcquire(&(HeadBuffer->Cursor));
		if ((Cursor + 1) & FSharedBuffer::RefInit)
		{
			ApplyThrottle();
			continue;
		}

		Cursor = Cursor >> FSharedBuffer::CursorShift;
		if (Cursor < 0)
		{
			Writer_RetireSharedBufferImpl();
			break;
		}

		uint32 PreSentBias = HeadBuffer->Size - GTailPreSent;
		if (uint32 Sendable = PreSentBias - Cursor)
		{
			uint8* Data = (uint8*)(UPTRINT(HeadBuffer) - PreSentBias);
			Writer_CacheData(Data, Sendable);
			GTailPreSent += Sendable;
		}

		break;
	}
}

////////////////////////////////////////////////////////////////////////////////
void Writer_InitializeSharedBuffers()
{
	FSharedBuffer* Buffer = Writer_CreateSharedBuffer();

	GTailBuffer = Buffer;
	GTailPreSent = 0;

	AtomicStoreRelease(&GSharedBuffer, Buffer);
}

////////////////////////////////////////////////////////////////////////////////
void Writer_ShutdownSharedBuffers()
{
}

} // namespace Private
} // namespace Trace
} // namespace UE

#endif // TRACE_PRIVATE_MINIMAL_ENABLED && UE_TRACE_ALLOW_IMPORTANTS
