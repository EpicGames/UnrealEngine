--- //UE4/Dev-Core/Engine/Source/Runtime/TraceLog/Private/Trace/LZ4/lz4.c.h	2019-11-12 08:53:15.000000000 -0500
+++ d:\depot\dev-core\Engine\Source\Runtime\TraceLog\Private\Trace\LZ4\lz4.c.h	2019-11-12 08:53:15.000000000 -0500
@@ -1621,7 +1621,9 @@
  * initial_check - check ip >= lencheck before start of loop.  Returns initial_error if so.
  * error (output) - error code.  Should be set to 0 before call.
  */
-typedef enum { loop_error = -2, initial_error = -1, ok = 0 } variable_length_error;
+// BEGIN EPIC MOD : The 'ok' identifier collides with symbols in the MacOS SDK
+typedef enum { loop_error = -2, initial_error = -1, length_ok = 0 } variable_length_error;
+// END EPIC MOD
 LZ4_FORCE_INLINE unsigned
 read_variable_length(const BYTE**ip, const BYTE* lencheck, int loop_check, int initial_check, variable_length_error* error)
 {
@@ -1721,7 +1723,9 @@
 
             /* decode literal length */
             if (length == RUN_MASK) {
-                variable_length_error error = ok;
+// BEGIN EPIC MOD : The 'ok' identifier collides with symbols in the MacOS SDK
+                variable_length_error error = length_ok;
+// END EPIC MOD
                 length += read_variable_length(&ip, iend-RUN_MASK, endOnInput, endOnInput, &error);
                 if (error == initial_error) { goto _output_error; }
                 if ((safeDecode) && unlikely((uptrval)(op)+length<(uptrval)(op))) { goto _output_error; } /* overflow detection */
@@ -1765,10 +1769,14 @@
             length = token & ML_MASK;
 
             if (length == ML_MASK) {
-              variable_length_error error = ok;
+// BEGIN EPIC MOD : The 'ok' identifier collides with symbols in the MacOS SDK
+              variable_length_error error = length_ok;
+// END EPIC MOD
               if ((checkOffset) && (unlikely(match + dictSize < lowPrefix))) { goto _output_error; } /* Error : offset outside buffers */
               length += read_variable_length(&ip, iend - LASTLITERALS + 1, endOnInput, 0, &error);
-              if (error != ok) { goto _output_error; }
+// BEGIN EPIC MOD : The 'ok' identifier collides with symbols in the MacOS SDK
+              if (error != length_ok) { goto _output_error; }
+// END EPIC MOD
                 if ((safeDecode) && unlikely((uptrval)(op)+length<(uptrval)op)) { goto _output_error; } /* overflow detection */
                 length += MINMATCH;
                 if (op + length >= oend - FASTLOOP_SAFE_DISTANCE) {
@@ -1890,7 +1898,9 @@
 
             /* decode literal length */
             if (length == RUN_MASK) {
-                variable_length_error error = ok;
+// BEGIN EPIC MOD : The 'ok' identifier collides with symbols in the MacOS SDK
+                variable_length_error error = length_ok;
+// END EPIC MOD
                 length += read_variable_length(&ip, iend-RUN_MASK, endOnInput, endOnInput, &error);
                 if (error == initial_error) { goto _output_error; }
                 if ((safeDecode) && unlikely((uptrval)(op)+length<(uptrval)(op))) { goto _output_error; } /* overflow detection */
@@ -1964,9 +1974,13 @@
 
     _copy_match:
             if (length == ML_MASK) {
-              variable_length_error error = ok;
+// BEGIN EPIC MOD : The 'ok' identifier collides with symbols in the MacOS SDK
+              variable_length_error error = length_ok;
+// END EPIC MOD
               length += read_variable_length(&ip, iend - LASTLITERALS + 1, endOnInput, 0, &error);
-              if (error != ok) goto _output_error;
+// BEGIN EPIC MOD : The 'ok' identifier collides with symbols in the MacOS SDK
+              if (error != length_ok) goto _output_error;
+// END EPIC MOD
                 if ((safeDecode) && unlikely((uptrval)(op)+length<(uptrval)op)) goto _output_error;   /* overflow detection */
             }
             length += MINMATCH;
--- //UE4/Dev-Core/Engine/Source/Runtime/TraceLog/Private/Trace/LZ4/lz4.c.h	2019-11-22 09:02:01.000000000 -0500
+++ d:\depot\dev-core\Engine\Source\Runtime\TraceLog\Private\Trace\LZ4\lz4.c.h	2019-11-22 09:02:01.000000000 -0500
@@ -242,6 +242,9 @@
 **************************************/
 #if defined(__cplusplus) || (defined (__STDC_VERSION__) && (__STDC_VERSION__ >= 199901L) /* C99 */)
 # include <stdint.h>
+
+LZ4_BEGIN_NAMESPACE // EPIC MOD : Wrap library in an optional namespace
+
   typedef  uint8_t BYTE;
   typedef uint16_t U16;
   typedef uint32_t U32;
@@ -611,20 +614,28 @@
 /*-************************************
 *  Internal Definitions used in Tests
 **************************************/
+LZ4_END_NAMESPACE // EPIC MOD : Wrap library in an optional namespace
+
 #if defined (__cplusplus)
 extern "C" {
 #endif
 
+LZ4_BEGIN_NAMESPACE // EPIC MOD : Wrap library in an optional namespace
+
 int LZ4_compress_forceExtDict (LZ4_stream_t* LZ4_dict, const char* source, char* dest, int srcSize);
 
 int LZ4_decompress_safe_forceExtDict(const char* source, char* dest,
                                      int compressedSize, int maxOutputSize,
                                      const void* dictStart, size_t dictSize);
 
+LZ4_END_NAMESPACE // EPIC MOD : Wrap library in an optional namespace
+
 #if defined (__cplusplus)
 }
 #endif
 
+LZ4_BEGIN_NAMESPACE // EPIC MOD : Wrap library in an optional namespace
+
 /*-******************************
 *  Compression functions
 ********************************/
@@ -2410,3 +2421,5 @@
 }
 
 #endif   /* LZ4_COMMONDEFS_ONLY */
+
+LZ4_END_NAMESPACE // EPIC MOD : Wrap library in an optional namespace
--- //UE4/Dev-Core/Engine/Source/Runtime/TraceLog/Private/Trace/LZ4/lz4.h	2019-11-12 08:53:15.000000000 -0500
+++ d:\depot\dev-core\Engine\Source\Runtime\TraceLog\Private\Trace\LZ4\lz4.h	2019-11-12 08:53:15.000000000 -0500
@@ -32,10 +32,21 @@
     - LZ4 homepage : http://www.lz4.org
     - LZ4 source repository : https://github.com/lz4/lz4
 */
+
 #if defined (__cplusplus)
 extern "C" {
 #endif
 
+// BEGIN EPIC MOD : Wrap library in an optional namespace
+#if defined(LZ4_NAMESPACE) && defined(__cplusplus)
+#  define LZ4_BEGIN_NAMESPACE   extern "C++" { namespace LZ4_NAMESPACE {
+#  define LZ4_END_NAMESPACE     } /* extern */ } /* namespace */
+#else
+#  define LZ4_BEGIN_NAMESPACE
+#  define LZ4_END_NAMESPACE
+#endif // LZ4_NAMESPACE
+// END EPIC MOD
+
 #ifndef LZ4_H_2983827168210
 #define LZ4_H_2983827168210
 
@@ -109,6 +120,8 @@
 #define LZ4_EXPAND_AND_QUOTE(str) LZ4_QUOTE(str)
 #define LZ4_VERSION_STRING LZ4_EXPAND_AND_QUOTE(LZ4_LIB_VERSION)
 
+LZ4_BEGIN_NAMESPACE // EPIC MOD : Wrap library in an optional namespace
+
 LZ4LIB_API int LZ4_versionNumber (void);  /**< library version number; useful to check dll version */
 LZ4LIB_API const char* LZ4_versionString (void);   /**< library version string; useful to check dll version */
 
@@ -397,6 +410,8 @@
  */
 LZ4LIB_API int LZ4_decompress_safe_usingDict (const char* src, char* dst, int srcSize, int dstCapcity, const char* dictStart, int dictSize);
 
+LZ4_END_NAMESPACE // EPIC MOD : Wrap library in an optional namespace
+
 #endif /* LZ4_H_2983827168210 */
 
 
@@ -435,6 +450,8 @@
 #define LZ4LIB_STATIC_API
 #endif
 
+LZ4_BEGIN_NAMESPACE // EPIC MOD : Wrap library in an optional namespace
+
 
 /*! LZ4_compress_fast_extState_fastReset() :
  *  A variant of LZ4_compress_fast_extState().
@@ -538,6 +555,8 @@
 #define LZ4_COMPRESS_INPLACE_MARGIN                           (LZ4_DISTANCE_MAX + 32)   /* LZ4_DISTANCE_MAX can be safely replaced by srcSize when it's smaller */
 #define LZ4_COMPRESS_INPLACE_BUFFER_SIZE(maxCompressedSize)   ((maxCompressedSize) + LZ4_COMPRESS_INPLACE_MARGIN)  /**< maxCompressedSize is generally LZ4_COMPRESSBOUND(inputSize), but can be set to any lower value, with the risk that compression can fail (return code 0(zero)) */
 
+LZ4_END_NAMESPACE // EPIC MOD : Wrap library in an optional namespace
+
 #endif   /* LZ4_STATIC_3504398509 */
 #endif   /* LZ4_STATIC_LINKING_ONLY */
 
@@ -560,6 +579,8 @@
 #if defined(__cplusplus) || (defined (__STDC_VERSION__) && (__STDC_VERSION__ >= 199901L) /* C99 */)
 #include <stdint.h>
 
+LZ4_BEGIN_NAMESPACE // EPIC MOD : Wrap library in an optional namespace
+
 typedef struct LZ4_stream_t_internal LZ4_stream_t_internal;
 struct LZ4_stream_t_internal {
     uint32_t hashTable[LZ4_HASH_SIZE_U32];
@@ -755,6 +776,8 @@
  */
 LZ4LIB_API void LZ4_resetStream (LZ4_stream_t* streamPtr);
 
+LZ4_END_NAMESPACE // EPIC MOD : Wrap library in an optional namespace
+
 
 #endif /* LZ4_H_98237428734687 */
 
