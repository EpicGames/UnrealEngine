/*
   LZ4 - Fast LZ compression algorithm
   Copyright (C) 2011-present, Yann Collet.

   BSD 2-Clause License (http://www.opensource.org/licenses/bsd-license.php)

   Redistribution and use in source and binary forms, with or without
   modification, are permitted provided that the following conditions are
   met:

       * Redistributions of source code must retain the above copyright
   notice, this list of conditions and the following disclaimer.
       * Redistributions in binary form must reproduce the above
   copyright notice, this list of conditions and the following disclaimer
   in the documentation and/or other materials provided with the
   distribution.

   THIS SOFTWARE IS PROVIDED BY THE COPYRIGHT HOLDERS AND CONTRIBUTORS
   "AS IS" AND ANY EXPRESS OR IMPLIED WARRANTIES, INCLUDING, BUT NOT
   LIMITED TO, THE IMPLIED WARRANTIES OF MERCHANTABILITY AND FITNESS FOR
   A PARTICULAR PURPOSE ARE DISCLAIMED. IN NO EVENT SHALL THE COPYRIGHT
   OWNER OR CONTRIBUTORS BE LIABLE FOR ANY DIRECT, INDIRECT, INCIDENTAL,
   SPECIAL, EXEMPLARY, OR CONSEQUENTIAL DAMAGES (INCLUDING, BUT NOT
   LIMITED TO, PROCUREMENT OF SUBSTITUTE GOODS OR SERVICES; LOSS OF USE,
   DATA, OR PROFITS; OR BUSINESS INTERRUPTION) HOWEVER CAUSED AND ON ANY
   THEORY OF LIABILITY, WHETHER IN CONTRACT, STRICT LIABILITY, OR TORT
   (INCLUDING NEGLIGENCE OR OTHERWISE) ARISING IN ANY WAY OUT OF THE USE
   OF THIS SOFTWARE, EVEN IF ADVISED OF THE POSSIBILITY OF SUCH DAMAGE.

   You can contact the author at :
    - LZ4 homepage : http://www.lz4.org
    - LZ4 source repository : https://github.com/lz4/lz4
*/

/*-************************************
*  Tuning parameters
**************************************/
/*
 * LZ4_HEAPMODE :
 * Select how default compression functions will allocate memory for their hash table,
 * in memory stack (0:default, fastest), or in memory heap (1:requires malloc()).
 */
#ifndef LZ4_HEAPMODE
#  define LZ4_HEAPMODE 0
#endif

/*
 * ACCELERATION_DEFAULT :
 * Select "acceleration" for LZ4_compress_fast() when parameter value <= 0
 */
#define ACCELERATION_DEFAULT 1


/*-************************************
*  CPU Feature Detection
**************************************/
/* LZ4_FORCE_MEMORY_ACCESS
 * By default, access to unaligned memory is controlled by `memcpy()`, which is safe and portable.
 * Unfortunately, on some target/compiler combinations, the generated assembly is sub-optimal.
 * The below switch allow to select different access method for improved performance.
 * Method 0 (default) : use `memcpy()`. Safe and portable.
 * Method 1 : `__packed` statement. It depends on compiler extension (ie, not portable).
 *            This method is safe if your compiler supports it, and *generally* as fast or faster than `memcpy`.
 * Method 2 : direct access. This method is portable but violate C standard.
 *            It can generate buggy code on targets which assembly generation depends on alignment.
 *            But in some circumstances, it's the only known way to get the most performance (ie GCC + ARMv6)
 * See https://fastcompression.blogspot.fr/2015/08/accessing-unaligned-memory.html for details.
 * Prefer these methods in priority order (0 > 1 > 2)
 */
#ifndef LZ4_FORCE_MEMORY_ACCESS   /* can be defined externally */
#  if defined(__GNUC__) && \
  ( defined(__ARM_ARCH_6__) || defined(__ARM_ARCH_6J__) || defined(__ARM_ARCH_6K__) \
  || defined(__ARM_ARCH_6Z__) || defined(__ARM_ARCH_6ZK__) || defined(__ARM_ARCH_6T2__) )
#    define LZ4_FORCE_MEMORY_ACCESS 2
#  elif (defined(__INTEL_COMPILER) && !defined(_WIN32)) || defined(__GNUC__)
#    define LZ4_FORCE_MEMORY_ACCESS 1
#  endif
#endif

/*
 * LZ4_FORCE_SW_BITCOUNT
 * Define this parameter if your target system or compiler does not support hardware bit count
 */
#if defined(_MSC_VER) && defined(_WIN32_WCE)   /* Visual Studio for WinCE doesn't support Hardware bit count */
#  define LZ4_FORCE_SW_BITCOUNT
#endif



/*-************************************
*  Dependency
**************************************/
/*
 * LZ4_SRC_INCLUDED:
 * Amalgamation flag, whether lz4.c is included
 */
#ifndef LZ4_SRC_INCLUDED
#  define LZ4_SRC_INCLUDED 1
#endif

#ifndef LZ4_STATIC_LINKING_ONLY
#define LZ4_STATIC_LINKING_ONLY
#endif

#ifndef LZ4_DISABLE_DEPRECATE_WARNINGS
#define LZ4_DISABLE_DEPRECATE_WARNINGS /* due to LZ4_decompress_safe_withPrefix64k */
#endif

#define LZ4_STATIC_LINKING_ONLY  /* LZ4_DISTANCE_MAX */
#include "lz4.h"
/* see also "memory routines" below */


/*-************************************
*  Compiler Options
**************************************/
#ifdef _MSC_VER    /* Visual Studio */
#  include <intrin.h>
#  pragma warning(disable : 4127)        /* disable: C4127: conditional expression is constant */
#  pragma warning(disable : 4293)        /* disable: C4293: too large shift (32-bits) */
#endif  /* _MSC_VER */

#ifndef LZ4_FORCE_INLINE
#  ifdef _MSC_VER    /* Visual Studio */
#    define LZ4_FORCE_INLINE static __forceinline
#  else
#    if defined (__cplusplus) || defined (__STDC_VERSION__) && __STDC_VERSION__ >= 199901L   /* C99 */
#      ifdef __GNUC__
#        define LZ4_FORCE_INLINE static inline __attribute__((always_inline))
#      else
#        define LZ4_FORCE_INLINE static inline
#      endif
#    else
#      define LZ4_FORCE_INLINE static
#    endif /* __STDC_VERSION__ */
#  endif  /* _MSC_VER */
#endif /* LZ4_FORCE_INLINE */

/* LZ4_FORCE_O2_GCC_PPC64LE and LZ4_FORCE_O2_INLINE_GCC_PPC64LE
 * gcc on ppc64le generates an unrolled SIMDized loop for LZ4_wildCopy8,
 * together with a simple 8-byte copy loop as a fall-back path.
 * However, this optimization hurts the decompression speed by >30%,
 * because the execution does not go to the optimized loop
 * for typical compressible data, and all of the preamble checks
 * before going to the fall-back path become useless overhead.
 * This optimization happens only with the -O3 flag, and -O2 generates
 * a simple 8-byte copy loop.
 * With gcc on ppc64le, all of the LZ4_decompress_* and LZ4_wildCopy8
 * functions are annotated with __attribute__((optimize("O2"))),
 * and also LZ4_wildCopy8 is forcibly inlined, so that the O2 attribute
 * of LZ4_wildCopy8 does not affect the compression speed.
 */
#if defined(__PPC64__) && defined(__LITTLE_ENDIAN__) && defined(__GNUC__) && !defined(__clang__)
#  define LZ4_FORCE_O2_GCC_PPC64LE __attribute__((optimize("O2")))
#  define LZ4_FORCE_O2_INLINE_GCC_PPC64LE __attribute__((optimize("O2"))) LZ4_FORCE_INLINE
#else
#  define LZ4_FORCE_O2_GCC_PPC64LE
#  define LZ4_FORCE_O2_INLINE_GCC_PPC64LE static
#endif

#if (defined(__GNUC__) && (__GNUC__ >= 3)) || (defined(__INTEL_COMPILER) && (__INTEL_COMPILER >= 800)) || defined(__clang__)
#  define expect(expr,value)    (__builtin_expect ((expr),(value)) )
#else
#  define expect(expr,value)    (expr)
#endif

#ifndef likely
#define likely(expr)     expect(int(expr) != 0, 1)
#endif
#ifndef unlikely
#define unlikely(expr)   expect(int(expr) != 0, 0)
#endif


/*-************************************
*  Memory routines
**************************************/
#include <stdlib.h>   /* malloc, calloc, free */
#define ALLOC(s)          malloc(s)
#define ALLOC_AND_ZERO(s) calloc(1,s)
#define FREEMEM(p)        free(p)
#include <string.h>   /* memset, memcpy */
#define MEM_INIT(p,v,s)   memset((p),(v),(s))


/*-************************************
*  Common Constants
**************************************/
#define MINMATCH 4

#define WILDCOPYLENGTH 8
#define LASTLITERALS   5   /* see ../doc/lz4_Block_format.md#parsing-restrictions */
#define MFLIMIT       12   /* see ../doc/lz4_Block_format.md#parsing-restrictions */
#define MATCH_SAFEGUARD_DISTANCE  ((2*WILDCOPYLENGTH) - MINMATCH)   /* ensure it's possible to write 2 x wildcopyLength without overflowing output buffer */
#define FASTLOOP_SAFE_DISTANCE 64
static const int LZ4_minLength = (MFLIMIT+1);

#define KB *(1 <<10)
#define MB *(1 <<20)
#define GB *(1U<<30)

#define LZ4_DISTANCE_ABSOLUTE_MAX 65535
#if (LZ4_DISTANCE_MAX > LZ4_DISTANCE_ABSOLUTE_MAX)   /* max supported by LZ4 format */
#  error "LZ4_DISTANCE_MAX is too big : must be <= 65535"
#endif

#define ML_BITS  4
#define ML_MASK  ((1U<<ML_BITS)-1)
#define RUN_BITS (8-ML_BITS)
#define RUN_MASK ((1U<<RUN_BITS)-1)


/*-************************************
*  Error detection
**************************************/
#if defined(LZ4_DEBUG) && (LZ4_DEBUG>=1)
#  include <assert.h>
#else
#  ifndef assert
#    define assert(condition) ((void)0)
#  endif
#endif

#define LZ4_STATIC_ASSERT(c)   { enum { LZ4_static_assert = 1/(int)(!!(c)) }; }   /* use after variable declarations */

#if defined(LZ4_DEBUG) && (LZ4_DEBUG>=2)
#  include <stdio.h>
static int g_debuglog_enable = 1;
#  define DEBUGLOG(l, ...) {                                  \
                if ((g_debuglog_enable) && (l<=LZ4_DEBUG)) {  \
                    fprintf(stderr, __FILE__ ": ");           \
                    fprintf(stderr, __VA_ARGS__);             \
                    fprintf(stderr, " \n");                   \
            }   }
#else
#  define DEBUGLOG(l, ...)      {}    /* disabled */
#endif


/*-************************************
*  Types
**************************************/
#if defined(__cplusplus) || (defined (__STDC_VERSION__) && (__STDC_VERSION__ >= 199901L) /* C99 */)
# include <stdint.h>

LZ4_BEGIN_NAMESPACE // EPIC MOD : Wrap library in an optional namespace

  typedef  uint8_t BYTE;
  typedef uint16_t U16;
  typedef uint32_t U32;
  typedef  int32_t S32;
  typedef uint64_t U64;
  typedef uintptr_t uptrval;
#else
  typedef unsigned char       BYTE;
  typedef unsigned short      U16;
  typedef unsigned int        U32;
  typedef   signed int        S32;
  typedef unsigned long long  U64;
  typedef size_t              uptrval;   /* generally true, except OpenVMS-64 */
#endif

#if defined(__x86_64__)
  typedef U64    reg_t;   /* 64-bits in x32 mode */
#else
  typedef size_t reg_t;   /* 32-bits in x32 mode */
#endif

typedef enum {
    notLimited = 0,
    limitedOutput = 1,
    fillOutput = 2
} limitedOutput_directive;


/*-************************************
*  Reading and writing into memory
**************************************/
static unsigned LZ4_isLittleEndian(void)
{
    const union { U32 u; BYTE c[4]; } one = { 1 };   /* don't use static : performance detrimental */
    return one.c[0];
}


#if defined(LZ4_FORCE_MEMORY_ACCESS) && (LZ4_FORCE_MEMORY_ACCESS==2)
/* lie to the compiler about data alignment; use with caution */

static U16 LZ4_read16(const void* memPtr) { return *(const U16*) memPtr; }
static U32 LZ4_read32(const void* memPtr) { return *(const U32*) memPtr; }
static reg_t LZ4_read_ARCH(const void* memPtr) { return *(const reg_t*) memPtr; }

static void LZ4_write16(void* memPtr, U16 value) { *(U16*)memPtr = value; }
static void LZ4_write32(void* memPtr, U32 value) { *(U32*)memPtr = value; }

#elif defined(LZ4_FORCE_MEMORY_ACCESS) && (LZ4_FORCE_MEMORY_ACCESS==1)

/* __pack instructions are safer, but compiler specific, hence potentially problematic for some compilers */
/* currently only defined for gcc and icc */
typedef union { U16 u16; U32 u32; reg_t uArch; } __attribute__((packed)) unalign;

static U16 LZ4_read16(const void* ptr) { return ((const unalign*)ptr)->u16; }
static U32 LZ4_read32(const void* ptr) { return ((const unalign*)ptr)->u32; }
static reg_t LZ4_read_ARCH(const void* ptr) { return ((const unalign*)ptr)->uArch; }

static void LZ4_write16(void* memPtr, U16 value) { ((unalign*)memPtr)->u16 = value; }
static void LZ4_write32(void* memPtr, U32 value) { ((unalign*)memPtr)->u32 = value; }

#else  /* safe and portable access using memcpy() */

static U16 LZ4_read16(const void* memPtr)
{
    U16 val; memcpy(&val, memPtr, sizeof(val)); return val;
}

static U32 LZ4_read32(const void* memPtr)
{
    U32 val; memcpy(&val, memPtr, sizeof(val)); return val;
}

static reg_t LZ4_read_ARCH(const void* memPtr)
{
    reg_t val; memcpy(&val, memPtr, sizeof(val)); return val;
}

static void LZ4_write16(void* memPtr, U16 value)
{
    memcpy(memPtr, &value, sizeof(value));
}

static void LZ4_write32(void* memPtr, U32 value)
{
    memcpy(memPtr, &value, sizeof(value));
}

#endif /* LZ4_FORCE_MEMORY_ACCESS */


static U16 LZ4_readLE16(const void* memPtr)
{
    if (LZ4_isLittleEndian()) {
        return LZ4_read16(memPtr);
    } else {
        const BYTE* p = (const BYTE*)memPtr;
        return (U16)((U16)p[0] + (p[1]<<8));
    }
}

static void LZ4_writeLE16(void* memPtr, U16 value)
{
    if (LZ4_isLittleEndian()) {
        LZ4_write16(memPtr, value);
    } else {
        BYTE* p = (BYTE*)memPtr;
        p[0] = (BYTE) value;
        p[1] = (BYTE)(value>>8);
    }
}

/* customized variant of memcpy, which can overwrite up to 8 bytes beyond dstEnd */
LZ4_FORCE_O2_INLINE_GCC_PPC64LE
void LZ4_wildCopy8(void* dstPtr, const void* srcPtr, void* dstEnd)
{
    BYTE* d = (BYTE*)dstPtr;
    const BYTE* s = (const BYTE*)srcPtr;
    BYTE* const e = (BYTE*)dstEnd;

    do { memcpy(d,s,8); d+=8; s+=8; } while (d<e);
}

static const unsigned inc32table[8] = {0, 1, 2,  1,  0,  4, 4, 4};
static const int      dec64table[8] = {0, 0, 0, -1, -4,  1, 2, 3};


#ifndef LZ4_FAST_DEC_LOOP
#  if defined(__i386__) || defined(__x86_64__)
#    define LZ4_FAST_DEC_LOOP 1
#  elif defined(__aarch64__) && !defined(__clang__)
     /* On aarch64, we disable this optimization for clang because on certain
      * mobile chipsets and clang, it reduces performance. For more information
      * refer to https://github.com/lz4/lz4/pull/707. */
#    define LZ4_FAST_DEC_LOOP 1
#  else
#    define LZ4_FAST_DEC_LOOP 0
#  endif
#endif

#if LZ4_FAST_DEC_LOOP

LZ4_FORCE_O2_INLINE_GCC_PPC64LE void
LZ4_memcpy_using_offset_base(BYTE* dstPtr, const BYTE* srcPtr, BYTE* dstEnd, const size_t offset)
{
    if (offset < 8) {
        dstPtr[0] = srcPtr[0];
        dstPtr[1] = srcPtr[1];
        dstPtr[2] = srcPtr[2];
        dstPtr[3] = srcPtr[3];
        srcPtr += inc32table[offset];
        memcpy(dstPtr+4, srcPtr, 4);
        srcPtr -= dec64table[offset];
        dstPtr += 8;
    } else {
        memcpy(dstPtr, srcPtr, 8);
        dstPtr += 8;
        srcPtr += 8;
    }

    LZ4_wildCopy8(dstPtr, srcPtr, dstEnd);
}

/* customized variant of memcpy, which can overwrite up to 32 bytes beyond dstEnd
 * this version copies two times 16 bytes (instead of one time 32 bytes)
 * because it must be compatible with offsets >= 16. */
LZ4_FORCE_O2_INLINE_GCC_PPC64LE void
LZ4_wildCopy32(void* dstPtr, const void* srcPtr, void* dstEnd)
{
    BYTE* d = (BYTE*)dstPtr;
    const BYTE* s = (const BYTE*)srcPtr;
    BYTE* const e = (BYTE*)dstEnd;

    do { memcpy(d,s,16); memcpy(d+16,s+16,16); d+=32; s+=32; } while (d<e);
}

/* LZ4_memcpy_using_offset()  presumes :
 * - dstEnd >= dstPtr + MINMATCH
 * - there is at least 8 bytes available to write after dstEnd */
LZ4_FORCE_O2_INLINE_GCC_PPC64LE void
LZ4_memcpy_using_offset(BYTE* dstPtr, const BYTE* srcPtr, BYTE* dstEnd, const size_t offset)
{
    BYTE v[8];

    assert(dstEnd >= dstPtr + MINMATCH);
    LZ4_write32(dstPtr, 0);   /* silence an msan warning when offset==0 */

    switch(offset) {
    case 1:
        memset(v, *srcPtr, 8);
        break;
    case 2:
        memcpy(v, srcPtr, 2);
        memcpy(&v[2], srcPtr, 2);
        memcpy(&v[4], &v[0], 4);
        break;
    case 4:
        memcpy(v, srcPtr, 4);
        memcpy(&v[4], srcPtr, 4);
        break;
    default:
        LZ4_memcpy_using_offset_base(dstPtr, srcPtr, dstEnd, offset);
        return;
    }

    memcpy(dstPtr, v, 8);
    dstPtr += 8;
    while (dstPtr < dstEnd) {
        memcpy(dstPtr, v, 8);
        dstPtr += 8;
    }
}
#endif


/*-************************************
*  Common functions
**************************************/
static unsigned LZ4_NbCommonBytes (reg_t val)
{
    if (LZ4_isLittleEndian()) {
        if (sizeof(val)==8) {
#       if defined(_MSC_VER) && defined(_WIN64) && !defined(LZ4_FORCE_SW_BITCOUNT)
            unsigned long r = 0;
            _BitScanForward64( &r, (U64)val );
            return (int)(r>>3);
#       elif (defined(__clang__) || (defined(__GNUC__) && (__GNUC__>=3))) && !defined(LZ4_FORCE_SW_BITCOUNT)
            return (unsigned)__builtin_ctzll((U64)val) >> 3;
#       else
            static const int DeBruijnBytePos[64] = { 0, 0, 0, 0, 0, 1, 1, 2,
                                                     0, 3, 1, 3, 1, 4, 2, 7,
                                                     0, 2, 3, 6, 1, 5, 3, 5,
                                                     1, 3, 4, 4, 2, 5, 6, 7,
                                                     7, 0, 1, 2, 3, 3, 4, 6,
                                                     2, 6, 5, 5, 3, 4, 5, 6,
                                                     7, 1, 2, 4, 6, 4, 4, 5,
                                                     7, 2, 6, 5, 7, 6, 7, 7 };
            return DeBruijnBytePos[((U64)((val & -(long long)val) * 0x0218A392CDABBD3FULL)) >> 58];
#       endif
        } else /* 32 bits */ {
#       if defined(_MSC_VER) && !defined(LZ4_FORCE_SW_BITCOUNT)
            unsigned long r;
            _BitScanForward( &r, (U32)val );
            return (int)(r>>3);
#       elif (defined(__clang__) || (defined(__GNUC__) && (__GNUC__>=3))) && !defined(LZ4_FORCE_SW_BITCOUNT)
            return (unsigned)__builtin_ctz((U32)val) >> 3;
#       else
            static const int DeBruijnBytePos[32] = { 0, 0, 3, 0, 3, 1, 3, 0,
                                                     3, 2, 2, 1, 3, 2, 0, 1,
                                                     3, 3, 1, 2, 2, 2, 2, 0,
                                                     3, 1, 2, 0, 1, 0, 1, 1 };
            return DeBruijnBytePos[((U32)((val & -(S32)val) * 0x077CB531U)) >> 27];
#       endif
        }
    } else   /* Big Endian CPU */ {
        if (sizeof(val)==8) {   /* 64-bits */
#       if defined(_MSC_VER) && defined(_WIN64) && !defined(LZ4_FORCE_SW_BITCOUNT)
            unsigned long r = 0;
            _BitScanReverse64( &r, val );
            return (unsigned)(r>>3);
#       elif (defined(__clang__) || (defined(__GNUC__) && (__GNUC__>=3))) && !defined(LZ4_FORCE_SW_BITCOUNT)
            return (unsigned)__builtin_clzll((U64)val) >> 3;
#       else
            static const U32 by32 = sizeof(val)*4;  /* 32 on 64 bits (goal), 16 on 32 bits.
                Just to avoid some static analyzer complaining about shift by 32 on 32-bits target.
                Note that this code path is never triggered in 32-bits mode. */
            unsigned r;
            if (!(val>>by32)) { r=4; } else { r=0; val>>=by32; }
            if (!(val>>16)) { r+=2; val>>=8; } else { val>>=24; }
            r += (!val);
            return r;
#       endif
        } else /* 32 bits */ {
#       if defined(_MSC_VER) && !defined(LZ4_FORCE_SW_BITCOUNT)
            unsigned long r = 0;
            _BitScanReverse( &r, (unsigned long)val );
            return (unsigned)(r>>3);
#       elif (defined(__clang__) || (defined(__GNUC__) && (__GNUC__>=3))) && !defined(LZ4_FORCE_SW_BITCOUNT)
            return (unsigned)__builtin_clz((U32)val) >> 3;
#       else
            unsigned r;
            if (!(val>>16)) { r=2; val>>=8; } else { r=0; val>>=24; }
            r += (!val);
            return r;
#       endif
        }
    }
}

#define STEPSIZE sizeof(reg_t)
LZ4_FORCE_INLINE
unsigned LZ4_count(const BYTE* pIn, const BYTE* pMatch, const BYTE* pInLimit)
{
    const BYTE* const pStart = pIn;

    if (likely(pIn < pInLimit-(STEPSIZE-1))) {
        reg_t const diff = LZ4_read_ARCH(pMatch) ^ LZ4_read_ARCH(pIn);
        if (!diff) {
            pIn+=STEPSIZE; pMatch+=STEPSIZE;
        } else {
            return LZ4_NbCommonBytes(diff);
    }   }

    while (likely(pIn < pInLimit-(STEPSIZE-1))) {
        reg_t const diff = LZ4_read_ARCH(pMatch) ^ LZ4_read_ARCH(pIn);
        if (!diff) { pIn+=STEPSIZE; pMatch+=STEPSIZE; continue; }
        pIn += LZ4_NbCommonBytes(diff);
        return (unsigned)(pIn - pStart);
    }

    if ((STEPSIZE==8) && (pIn<(pInLimit-3)) && (LZ4_read32(pMatch) == LZ4_read32(pIn))) { pIn+=4; pMatch+=4; }
    if ((pIn<(pInLimit-1)) && (LZ4_read16(pMatch) == LZ4_read16(pIn))) { pIn+=2; pMatch+=2; }
    if ((pIn<pInLimit) && (*pMatch == *pIn)) pIn++;
    return (unsigned)(pIn - pStart);
}


#ifndef LZ4_COMMONDEFS_ONLY
/*-************************************
*  Local Constants
**************************************/
static const int LZ4_64Klimit = ((64 KB) + (MFLIMIT-1));
static const U32 LZ4_skipTrigger = 6;  /* Increase this value ==> compression run slower on incompressible data */


/*-************************************
*  Local Structures and types
**************************************/
typedef enum { clearedTable = 0, byPtr, byU32, byU16 } tableType_t;

/**
 * This enum distinguishes several different modes of accessing previous
 * content in the stream.
 *
 * - noDict        : There is no preceding content.
 * - withPrefix64k : Table entries up to ctx->dictSize before the current blob
 *                   blob being compressed are valid and refer to the preceding
 *                   content (of length ctx->dictSize), which is available
 *                   contiguously preceding in memory the content currently
 *                   being compressed.
 * - usingExtDict  : Like withPrefix64k, but the preceding content is somewhere
 *                   else in memory, starting at ctx->dictionary with length
 *                   ctx->dictSize.
 * - usingDictCtx  : Like usingExtDict, but everything concerning the preceding
 *                   content is in a separate context, pointed to by
 *                   ctx->dictCtx. ctx->dictionary, ctx->dictSize, and table
 *                   entries in the current context that refer to positions
 *                   preceding the beginning of the current compression are
 *                   ignored. Instead, ctx->dictCtx->dictionary and ctx->dictCtx
 *                   ->dictSize describe the location and size of the preceding
 *                   content, and matches are found by looking in the ctx
 *                   ->dictCtx->hashTable.
 */
typedef enum { noDict = 0, withPrefix64k, usingExtDict, usingDictCtx } dict_directive;
typedef enum { noDictIssue = 0, dictSmall } dictIssue_directive;


/*-************************************
*  Local Utils
**************************************/
int LZ4_versionNumber (void) { return LZ4_VERSION_NUMBER; }
const char* LZ4_versionString(void) { return LZ4_VERSION_STRING; }
int LZ4_compressBound(int isize)  { return LZ4_COMPRESSBOUND(isize); }
int LZ4_sizeofState() { return LZ4_STREAMSIZE; }


/*-************************************
*  Internal Definitions used in Tests
**************************************/
LZ4_END_NAMESPACE // EPIC MOD : Wrap library in an optional namespace

#if defined (__cplusplus)
extern "C" {
#endif

LZ4_BEGIN_NAMESPACE // EPIC MOD : Wrap library in an optional namespace

int LZ4_compress_forceExtDict (LZ4_stream_t* LZ4_dict, const char* source, char* dest, int srcSize);

int LZ4_decompress_safe_forceExtDict(const char* source, char* dest,
                                     int compressedSize, int maxOutputSize,
                                     const void* dictStart, size_t dictSize);

LZ4_END_NAMESPACE // EPIC MOD : Wrap library in an optional namespace

#if defined (__cplusplus)
}
#endif

LZ4_BEGIN_NAMESPACE // EPIC MOD : Wrap library in an optional namespace

/*-******************************
*  Compression functions
********************************/
static U32 LZ4_hash4(U32 sequence, tableType_t const tableType)
{
    if (tableType == byU16)
        return ((sequence * 2654435761U) >> ((MINMATCH*8)-(LZ4_HASHLOG+1)));
    else
        return ((sequence * 2654435761U) >> ((MINMATCH*8)-LZ4_HASHLOG));
}

static U32 LZ4_hash5(U64 sequence, tableType_t const tableType)
{
    const U32 hashLog = (tableType == byU16) ? LZ4_HASHLOG+1 : LZ4_HASHLOG;
    if (LZ4_isLittleEndian()) {
        const U64 prime5bytes = 889523592379ULL;
        return (U32)(((sequence << 24) * prime5bytes) >> (64 - hashLog));
    } else {
        const U64 prime8bytes = 11400714785074694791ULL;
        return (U32)(((sequence >> 24) * prime8bytes) >> (64 - hashLog));
    }
}

LZ4_FORCE_INLINE U32 LZ4_hashPosition(const void* const p, tableType_t const tableType)
{
    if ((sizeof(reg_t)==8) && (tableType != byU16)) return LZ4_hash5(LZ4_read_ARCH(p), tableType);
    return LZ4_hash4(LZ4_read32(p), tableType);
}

static void LZ4_clearHash(U32 h, void* tableBase, tableType_t const tableType)
{
    switch (tableType)
    {
    default: /* fallthrough */
    case clearedTable: { /* illegal! */ assert(0); return; }
    case byPtr: { const BYTE** hashTable = (const BYTE**)tableBase; hashTable[h] = NULL; return; }
    case byU32: { U32* hashTable = (U32*) tableBase; hashTable[h] = 0; return; }
    case byU16: { U16* hashTable = (U16*) tableBase; hashTable[h] = 0; return; }
    }
}

static void LZ4_putIndexOnHash(U32 idx, U32 h, void* tableBase, tableType_t const tableType)
{
    switch (tableType)
    {
    default: /* fallthrough */
    case clearedTable: /* fallthrough */
    case byPtr: { /* illegal! */ assert(0); return; }
    case byU32: { U32* hashTable = (U32*) tableBase; hashTable[h] = idx; return; }
    case byU16: { U16* hashTable = (U16*) tableBase; assert(idx < 65536); hashTable[h] = (U16)idx; return; }
    }
}

static void LZ4_putPositionOnHash(const BYTE* p, U32 h,
                                  void* tableBase, tableType_t const tableType,
                            const BYTE* srcBase)
{
    switch (tableType)
    {
    case clearedTable: { /* illegal! */ assert(0); return; }
    case byPtr: { const BYTE** hashTable = (const BYTE**)tableBase; hashTable[h] = p; return; }
    case byU32: { U32* hashTable = (U32*) tableBase; hashTable[h] = (U32)(p-srcBase); return; }
    case byU16: { U16* hashTable = (U16*) tableBase; hashTable[h] = (U16)(p-srcBase); return; }
    }
}

LZ4_FORCE_INLINE void LZ4_putPosition(const BYTE* p, void* tableBase, tableType_t tableType, const BYTE* srcBase)
{
    U32 const h = LZ4_hashPosition(p, tableType);
    LZ4_putPositionOnHash(p, h, tableBase, tableType, srcBase);
}

/* LZ4_getIndexOnHash() :
 * Index of match position registered in hash table.
 * hash position must be calculated by using base+index, or dictBase+index.
 * Assumption 1 : only valid if tableType == byU32 or byU16.
 * Assumption 2 : h is presumed valid (within limits of hash table)
 */
static U32 LZ4_getIndexOnHash(U32 h, const void* tableBase, tableType_t tableType)
{
    LZ4_STATIC_ASSERT(LZ4_MEMORY_USAGE > 2);
    if (tableType == byU32) {
        const U32* const hashTable = (const U32*) tableBase;
        assert(h < (1U << (LZ4_MEMORY_USAGE-2)));
        return hashTable[h];
    }
    if (tableType == byU16) {
        const U16* const hashTable = (const U16*) tableBase;
        assert(h < (1U << (LZ4_MEMORY_USAGE-1)));
        return hashTable[h];
    }
    assert(0); return 0;  /* forbidden case */
}

static const BYTE* LZ4_getPositionOnHash(U32 h, const void* tableBase, tableType_t tableType, const BYTE* srcBase)
{
    if (tableType == byPtr) { const BYTE* const* hashTable = (const BYTE* const*) tableBase; return hashTable[h]; }
    if (tableType == byU32) { const U32* const hashTable = (const U32*) tableBase; return hashTable[h] + srcBase; }
    { const U16* const hashTable = (const U16*) tableBase; return hashTable[h] + srcBase; }   /* default, to ensure a return */
}

LZ4_FORCE_INLINE const BYTE*
LZ4_getPosition(const BYTE* p,
                const void* tableBase, tableType_t tableType,
                const BYTE* srcBase)
{
    U32 const h = LZ4_hashPosition(p, tableType);
    return LZ4_getPositionOnHash(h, tableBase, tableType, srcBase);
}

LZ4_FORCE_INLINE void
LZ4_prepareTable(LZ4_stream_t_internal* const cctx,
           const int inputSize,
           const tableType_t tableType) {
    /* If compression failed during the previous step, then the context
     * is marked as dirty, therefore, it has to be fully reset.
     */
    if (cctx->dirty) {
        DEBUGLOG(5, "LZ4_prepareTable: Full reset for %p", cctx);
        MEM_INIT(cctx, 0, sizeof(LZ4_stream_t_internal));
        return;
    }

    /* If the table hasn't been used, it's guaranteed to be zeroed out, and is
     * therefore safe to use no matter what mode we're in. Otherwise, we figure
     * out if it's safe to leave as is or whether it needs to be reset.
     */
    if (cctx->tableType != clearedTable) {
        assert(inputSize >= 0);
        if (cctx->tableType != tableType
          || ((tableType == byU16) && cctx->currentOffset + (unsigned)inputSize >= 0xFFFFU)
          || ((tableType == byU32) && cctx->currentOffset > 1 GB)
          || tableType == byPtr
          || inputSize >= 4 KB)
        {
            DEBUGLOG(4, "LZ4_prepareTable: Resetting table in %p", cctx);
            MEM_INIT(cctx->hashTable, 0, LZ4_HASHTABLESIZE);
            cctx->currentOffset = 0;
            cctx->tableType = clearedTable;
        } else {
            DEBUGLOG(4, "LZ4_prepareTable: Re-use hash table (no reset)");
        }
    }

    /* Adding a gap, so all previous entries are > LZ4_DISTANCE_MAX back, is faster
     * than compressing without a gap. However, compressing with
     * currentOffset == 0 is faster still, so we preserve that case.
     */
    if (cctx->currentOffset != 0 && tableType == byU32) {
        DEBUGLOG(5, "LZ4_prepareTable: adding 64KB to currentOffset");
        cctx->currentOffset += 64 KB;
    }

    /* Finally, clear history */
    cctx->dictCtx = NULL;
    cctx->dictionary = NULL;
    cctx->dictSize = 0;
}

/** LZ4_compress_generic() :
    inlined, to ensure branches are decided at compilation time */
LZ4_FORCE_INLINE int LZ4_compress_generic(
                 LZ4_stream_t_internal* const cctx,
                 const char* const source,
                 char* const dest,
                 const int inputSize,
                 int *inputConsumed, /* only written when outputDirective == fillOutput */
                 const int maxOutputSize,
                 const limitedOutput_directive outputDirective,
                 const tableType_t tableType,
                 const dict_directive dictDirective,
                 const dictIssue_directive dictIssue,
                 const int acceleration)
{
    int result;
    const BYTE* ip = (const BYTE*) source;

    U32 const startIndex = cctx->currentOffset;
    const BYTE* base = (const BYTE*) source - startIndex;
    const BYTE* lowLimit;

    const LZ4_stream_t_internal* dictCtx = (const LZ4_stream_t_internal*) cctx->dictCtx;
    const BYTE* const dictionary =
        dictDirective == usingDictCtx ? dictCtx->dictionary : cctx->dictionary;
    const U32 dictSize =
        dictDirective == usingDictCtx ? dictCtx->dictSize : cctx->dictSize;
    const U32 dictDelta = (dictDirective == usingDictCtx) ? startIndex - dictCtx->currentOffset : 0;   /* make indexes in dictCtx comparable with index in current context */

    int const maybe_extMem = (dictDirective == usingExtDict) || (dictDirective == usingDictCtx);
    U32 const prefixIdxLimit = startIndex - dictSize;   /* used when dictDirective == dictSmall */
    const BYTE* const dictEnd = dictionary + dictSize;
    const BYTE* anchor = (const BYTE*) source;
    const BYTE* const iend = ip + inputSize;
    const BYTE* const mflimitPlusOne = iend - MFLIMIT + 1;
    const BYTE* const matchlimit = iend - LASTLITERALS;

    /* the dictCtx currentOffset is indexed on the start of the dictionary,
     * while a dictionary in the current context precedes the currentOffset */
    const BYTE* dictBase = (dictDirective == usingDictCtx) ?
                            dictionary + dictSize - dictCtx->currentOffset :
                            dictionary + dictSize - startIndex;

    BYTE* op = (BYTE*) dest;
    BYTE* const olimit = op + maxOutputSize;

    U32 offset = 0;
    U32 forwardH;

    DEBUGLOG(5, "LZ4_compress_generic: srcSize=%i, tableType=%u", inputSize, tableType);
    /* If init conditions are not met, we don't have to mark stream
     * as having dirty context, since no action was taken yet */
    if (outputDirective == fillOutput && maxOutputSize < 1) { return 0; } /* Impossible to store anything */
    if ((U32)inputSize > (U32)LZ4_MAX_INPUT_SIZE) { return 0; }           /* Unsupported inputSize, too large (or negative) */
    if ((tableType == byU16) && (inputSize>=LZ4_64Klimit)) { return 0; }  /* Size too large (not within 64K limit) */
    if (tableType==byPtr) assert(dictDirective==noDict);      /* only supported use case with byPtr */
    assert(acceleration >= 1);

    lowLimit = (const BYTE*)source - (dictDirective == withPrefix64k ? dictSize : 0);

    /* Update context state */
    if (dictDirective == usingDictCtx) {
        /* Subsequent linked blocks can't use the dictionary. */
        /* Instead, they use the block we just compressed. */
        cctx->dictCtx = NULL;
        cctx->dictSize = (U32)inputSize;
    } else {
        cctx->dictSize += (U32)inputSize;
    }
    cctx->currentOffset += (U32)inputSize;
    cctx->tableType = (U16)tableType;

    if (inputSize<LZ4_minLength) goto _last_literals;        /* Input too small, no compression (all literals) */

    /* First Byte */
    LZ4_putPosition(ip, cctx->hashTable, tableType, base);
    ip++; forwardH = LZ4_hashPosition(ip, tableType);

    /* Main Loop */
    for ( ; ; ) {
        const BYTE* match;
        BYTE* token;
        const BYTE* filledIp;

        /* Find a match */
        if (tableType == byPtr) {
            const BYTE* forwardIp = ip;
            int step = 1;
            int searchMatchNb = acceleration << LZ4_skipTrigger;
            do {
                U32 const h = forwardH;
                ip = forwardIp;
                forwardIp += step;
                step = (searchMatchNb++ >> LZ4_skipTrigger);

                if (unlikely(forwardIp > mflimitPlusOne)) goto _last_literals;
                assert(ip < mflimitPlusOne);

                match = LZ4_getPositionOnHash(h, cctx->hashTable, tableType, base);
                forwardH = LZ4_hashPosition(forwardIp, tableType);
                LZ4_putPositionOnHash(ip, h, cctx->hashTable, tableType, base);

            } while ( (match+LZ4_DISTANCE_MAX < ip)
                   || (LZ4_read32(match) != LZ4_read32(ip)) );

        } else {   /* byU32, byU16 */

            const BYTE* forwardIp = ip;
            int step = 1;
            int searchMatchNb = acceleration << LZ4_skipTrigger;
            do {
                U32 const h = forwardH;
                U32 const current = (U32)(forwardIp - base);
                U32 matchIndex = LZ4_getIndexOnHash(h, cctx->hashTable, tableType);
                assert(matchIndex <= current);
                assert(forwardIp - base < (ptrdiff_t)(2 GB - 1));
                ip = forwardIp;
                forwardIp += step;
                step = (searchMatchNb++ >> LZ4_skipTrigger);

                if (unlikely(forwardIp > mflimitPlusOne)) goto _last_literals;
                assert(ip < mflimitPlusOne);

                if (dictDirective == usingDictCtx) {
                    if (matchIndex < startIndex) {
                        /* there was no match, try the dictionary */
                        assert(tableType == byU32);
                        matchIndex = LZ4_getIndexOnHash(h, dictCtx->hashTable, byU32);
                        match = dictBase + matchIndex;
                        matchIndex += dictDelta;   /* make dictCtx index comparable with current context */
                        lowLimit = dictionary;
                    } else {
                        match = base + matchIndex;
                        lowLimit = (const BYTE*)source;
                    }
                } else if (dictDirective==usingExtDict) {
                    if (matchIndex < startIndex) {
                        DEBUGLOG(7, "extDict candidate: matchIndex=%5u  <  startIndex=%5u", matchIndex, startIndex);
                        assert(startIndex - matchIndex >= MINMATCH);
                        match = dictBase + matchIndex;
                        lowLimit = dictionary;
                    } else {
                        match = base + matchIndex;
                        lowLimit = (const BYTE*)source;
                    }
                } else {   /* single continuous memory segment */
                    match = base + matchIndex;
                }
                forwardH = LZ4_hashPosition(forwardIp, tableType);
                LZ4_putIndexOnHash(current, h, cctx->hashTable, tableType);

                DEBUGLOG(7, "candidate at pos=%u  (offset=%u \n", matchIndex, current - matchIndex);
                if ((dictIssue == dictSmall) && (matchIndex < prefixIdxLimit)) { continue; }    /* match outside of valid area */
                assert(matchIndex < current);
                if ( ((tableType != byU16) || (LZ4_DISTANCE_MAX < LZ4_DISTANCE_ABSOLUTE_MAX))
                  && (matchIndex+LZ4_DISTANCE_MAX < current)) {
                    continue;
                } /* too far */
                assert((current - matchIndex) <= LZ4_DISTANCE_MAX);  /* match now expected within distance */

                if (LZ4_read32(match) == LZ4_read32(ip)) {
                    if (maybe_extMem) offset = current - matchIndex;
                    break;   /* match found */
                }

            } while(1);
        }

        /* Catch up */
        filledIp = ip;
        while (((ip>anchor) & (match > lowLimit)) && (unlikely(ip[-1]==match[-1]))) { ip--; match--; }

        /* Encode Literals */
        {   unsigned const litLength = (unsigned)(ip - anchor);
            token = op++;
            if ((outputDirective == limitedOutput) &&  /* Check output buffer overflow */
                (unlikely(op + litLength + (2 + 1 + LASTLITERALS) + (litLength/255) > olimit)) ) {
                return 0;   /* cannot compress within `dst` budget. Stored indexes in hash table are nonetheless fine */
            }
            if ((outputDirective == fillOutput) &&
                (unlikely(op + (litLength+240)/255 /* litlen */ + litLength /* literals */ + 2 /* offset */ + 1 /* token */ + MFLIMIT - MINMATCH /* min last literals so last match is <= end - MFLIMIT */ > olimit))) {
                op--;
                goto _last_literals;
            }
            if (litLength >= RUN_MASK) {
                int len = (int)(litLength - RUN_MASK);
                *token = (RUN_MASK<<ML_BITS);
                for(; len >= 255 ; len-=255) *op++ = 255;
                *op++ = (BYTE)len;
            }
            else *token = (BYTE)(litLength<<ML_BITS);

            /* Copy Literals */
            LZ4_wildCopy8(op, anchor, op+litLength);
            op+=litLength;
            DEBUGLOG(6, "seq.start:%i, literals=%u, match.start:%i",
                        (int)(anchor-(const BYTE*)source), litLength, (int)(ip-(const BYTE*)source));
        }

_next_match:
        /* at this stage, the following variables must be correctly set :
         * - ip : at start of LZ operation
         * - match : at start of previous pattern occurence; can be within current prefix, or within extDict
         * - offset : if maybe_ext_memSegment==1 (constant)
         * - lowLimit : must be == dictionary to mean "match is within extDict"; must be == source otherwise
         * - token and *token : position to write 4-bits for match length; higher 4-bits for literal length supposed already written
         */

        if ((outputDirective == fillOutput) &&
            (op + 2 /* offset */ + 1 /* token */ + MFLIMIT - MINMATCH /* min last literals so last match is <= end - MFLIMIT */ > olimit)) {
            /* the match was too close to the end, rewind and go to last literals */
            op = token;
            goto _last_literals;
        }

        /* Encode Offset */
        if (maybe_extMem) {   /* static test */
            DEBUGLOG(6, "             with offset=%u  (ext if > %i)", offset, (int)(ip - (const BYTE*)source));
            assert(offset <= LZ4_DISTANCE_MAX && offset > 0);
            LZ4_writeLE16(op, (U16)offset); op+=2;
        } else  {
            DEBUGLOG(6, "             with offset=%u  (same segment)", (U32)(ip - match));
            assert(ip-match <= LZ4_DISTANCE_MAX);
            LZ4_writeLE16(op, (U16)(ip - match)); op+=2;
        }

        /* Encode MatchLength */
        {   unsigned matchCode;

            if ( (dictDirective==usingExtDict || dictDirective==usingDictCtx)
              && (lowLimit==dictionary) /* match within extDict */ ) {
                const BYTE* limit = ip + (dictEnd-match);
                assert(dictEnd > match);
                if (limit > matchlimit) limit = matchlimit;
                matchCode = LZ4_count(ip+MINMATCH, match+MINMATCH, limit);
                ip += (size_t)matchCode + MINMATCH;
                if (ip==limit) {
                    unsigned const more = LZ4_count(limit, (const BYTE*)source, matchlimit);
                    matchCode += more;
                    ip += more;
                }
                DEBUGLOG(6, "             with matchLength=%u starting in extDict", matchCode+MINMATCH);
            } else {
                matchCode = LZ4_count(ip+MINMATCH, match+MINMATCH, matchlimit);
                ip += (size_t)matchCode + MINMATCH;
                DEBUGLOG(6, "             with matchLength=%u", matchCode+MINMATCH);
            }

            if ((outputDirective) &&    /* Check output buffer overflow */
                (unlikely(op + (1 + LASTLITERALS) + (matchCode+240)/255 > olimit)) ) {
                if (outputDirective == fillOutput) {
                    /* Match description too long : reduce it */
                    U32 newMatchCode = 15 /* in token */ - 1 /* to avoid needing a zero byte */ + ((U32)(olimit - op) - 1 - LASTLITERALS) * 255;
                    ip -= matchCode - newMatchCode;
                    assert(newMatchCode < matchCode);
                    matchCode = newMatchCode;
                    if (unlikely(ip <= filledIp)) {
                        /* We have already filled up to filledIp so if ip ends up less than filledIp
                         * we have positions in the hash table beyond the current position. This is
                         * a problem if we reuse the hash table. So we have to remove these positions
                         * from the hash table.
                         */
                        const BYTE* ptr;
                        DEBUGLOG(5, "Clearing %u positions", (U32)(filledIp - ip));
                        for (ptr = ip; ptr <= filledIp; ++ptr) {
                            U32 const h = LZ4_hashPosition(ptr, tableType);
                            LZ4_clearHash(h, cctx->hashTable, tableType);
                        }
                    }
                } else {
                    assert(outputDirective == limitedOutput);
                    return 0;   /* cannot compress within `dst` budget. Stored indexes in hash table are nonetheless fine */
                }
            }
            if (matchCode >= ML_MASK) {
                *token += ML_MASK;
                matchCode -= ML_MASK;
                LZ4_write32(op, 0xFFFFFFFF);
                while (matchCode >= 4*255) {
                    op+=4;
                    LZ4_write32(op, 0xFFFFFFFF);
                    matchCode -= 4*255;
                }
                op += matchCode / 255;
                *op++ = (BYTE)(matchCode % 255);
            } else
                *token += (BYTE)(matchCode);
        }
        /* Ensure we have enough space for the last literals. */
        assert(!(outputDirective == fillOutput && op + 1 + LASTLITERALS > olimit));

        anchor = ip;

        /* Test end of chunk */
        if (ip >= mflimitPlusOne) break;

        /* Fill table */
        LZ4_putPosition(ip-2, cctx->hashTable, tableType, base);

        /* Test next position */
        if (tableType == byPtr) {

            match = LZ4_getPosition(ip, cctx->hashTable, tableType, base);
            LZ4_putPosition(ip, cctx->hashTable, tableType, base);
            if ( (match+LZ4_DISTANCE_MAX >= ip)
              && (LZ4_read32(match) == LZ4_read32(ip)) )
            { token=op++; *token=0; goto _next_match; }

        } else {   /* byU32, byU16 */

            U32 const h = LZ4_hashPosition(ip, tableType);
            U32 const current = (U32)(ip-base);
            U32 matchIndex = LZ4_getIndexOnHash(h, cctx->hashTable, tableType);
            assert(matchIndex < current);
            if (dictDirective == usingDictCtx) {
                if (matchIndex < startIndex) {
                    /* there was no match, try the dictionary */
                    matchIndex = LZ4_getIndexOnHash(h, dictCtx->hashTable, byU32);
                    match = dictBase + matchIndex;
                    lowLimit = dictionary;   /* required for match length counter */
                    matchIndex += dictDelta;
                } else {
                    match = base + matchIndex;
                    lowLimit = (const BYTE*)source;  /* required for match length counter */
                }
            } else if (dictDirective==usingExtDict) {
                if (matchIndex < startIndex) {
                    match = dictBase + matchIndex;
                    lowLimit = dictionary;   /* required for match length counter */
                } else {
                    match = base + matchIndex;
                    lowLimit = (const BYTE*)source;   /* required for match length counter */
                }
            } else {   /* single memory segment */
                match = base + matchIndex;
            }
            LZ4_putIndexOnHash(current, h, cctx->hashTable, tableType);
            assert(matchIndex < current);
            if ( ((dictIssue==dictSmall) ? (matchIndex >= prefixIdxLimit) : 1)
              && (((tableType==byU16) && (LZ4_DISTANCE_MAX == LZ4_DISTANCE_ABSOLUTE_MAX)) ? 1 : (matchIndex+LZ4_DISTANCE_MAX >= current))
              && (LZ4_read32(match) == LZ4_read32(ip)) ) {
                token=op++;
                *token=0;
                if (maybe_extMem) offset = current - matchIndex;
                DEBUGLOG(6, "seq.start:%i, literals=%u, match.start:%i",
                            (int)(anchor-(const BYTE*)source), 0, (int)(ip-(const BYTE*)source));
                goto _next_match;
            }
        }

        /* Prepare next loop */
        forwardH = LZ4_hashPosition(++ip, tableType);

    }

_last_literals:
    /* Encode Last Literals */
    {   size_t lastRun = (size_t)(iend - anchor);
        if ( (outputDirective) &&  /* Check output buffer overflow */
            (op + lastRun + 1 + ((lastRun+255-RUN_MASK)/255) > olimit)) {
            if (outputDirective == fillOutput) {
                /* adapt lastRun to fill 'dst' */
                assert(olimit >= op);
                lastRun  = (size_t)(olimit-op) - 1;
                lastRun -= (lastRun+240)/255;
            } else {
                assert(outputDirective == limitedOutput);
                return 0;   /* cannot compress within `dst` budget. Stored indexes in hash table are nonetheless fine */
            }
        }
        if (lastRun >= RUN_MASK) {
            size_t accumulator = lastRun - RUN_MASK;
            *op++ = RUN_MASK << ML_BITS;
            for(; accumulator >= 255 ; accumulator-=255) *op++ = 255;
            *op++ = (BYTE) accumulator;
        } else {
            *op++ = (BYTE)(lastRun<<ML_BITS);
        }
        memcpy(op, anchor, lastRun);
        ip = anchor + lastRun;
        op += lastRun;
    }

    if (outputDirective == fillOutput) {
        *inputConsumed = (int) (((const char*)ip)-source);
    }
    DEBUGLOG(5, "LZ4_compress_generic: compressed %i bytes into %i bytes", inputSize, (int)(((char*)op) - dest));
    result = (int)(((char*)op) - dest);
    assert(result > 0);
    return result;
}


int LZ4_compress_fast_extState(void* state, const char* source, char* dest, int inputSize, int maxOutputSize, int acceleration)
{
    LZ4_stream_t_internal* const ctx = & LZ4_initStream(state, sizeof(LZ4_stream_t)) -> internal_donotuse;
    assert(ctx != NULL);
    if (acceleration < 1) acceleration = ACCELERATION_DEFAULT;
    if (maxOutputSize >= LZ4_compressBound(inputSize)) {
        if (inputSize < LZ4_64Klimit) {
            return LZ4_compress_generic(ctx, source, dest, inputSize, NULL, 0, notLimited, byU16, noDict, noDictIssue, acceleration);
        } else {
            const tableType_t tableType = ((sizeof(void*)==4) && ((uptrval)source > LZ4_DISTANCE_MAX)) ? byPtr : byU32;
            return LZ4_compress_generic(ctx, source, dest, inputSize, NULL, 0, notLimited, tableType, noDict, noDictIssue, acceleration);
        }
    } else {
        if (inputSize < LZ4_64Klimit) {
            return LZ4_compress_generic(ctx, source, dest, inputSize, NULL, maxOutputSize, limitedOutput, byU16, noDict, noDictIssue, acceleration);
        } else {
            const tableType_t tableType = ((sizeof(void*)==4) && ((uptrval)source > LZ4_DISTANCE_MAX)) ? byPtr : byU32;
            return LZ4_compress_generic(ctx, source, dest, inputSize, NULL, maxOutputSize, limitedOutput, tableType, noDict, noDictIssue, acceleration);
        }
    }
}

/**
 * LZ4_compress_fast_extState_fastReset() :
 * A variant of LZ4_compress_fast_extState().
 *
 * Using this variant avoids an expensive initialization step. It is only safe
 * to call if the state buffer is known to be correctly initialized already
 * (see comment in lz4.h on LZ4_resetStream_fast() for a definition of
 * "correctly initialized").
 */
int LZ4_compress_fast_extState_fastReset(void* state, const char* src, char* dst, int srcSize, int dstCapacity, int acceleration)
{
    LZ4_stream_t_internal* ctx = &((LZ4_stream_t*)state)->internal_donotuse;
    if (acceleration < 1) acceleration = ACCELERATION_DEFAULT;

    if (dstCapacity >= LZ4_compressBound(srcSize)) {
        if (srcSize < LZ4_64Klimit) {
            const tableType_t tableType = byU16;
            LZ4_prepareTable(ctx, srcSize, tableType);
            if (ctx->currentOffset) {
                return LZ4_compress_generic(ctx, src, dst, srcSize, NULL, 0, notLimited, tableType, noDict, dictSmall, acceleration);
            } else {
                return LZ4_compress_generic(ctx, src, dst, srcSize, NULL, 0, notLimited, tableType, noDict, noDictIssue, acceleration);
            }
        } else {
            const tableType_t tableType = ((sizeof(void*)==4) && ((uptrval)src > LZ4_DISTANCE_MAX)) ? byPtr : byU32;
            LZ4_prepareTable(ctx, srcSize, tableType);
            return LZ4_compress_generic(ctx, src, dst, srcSize, NULL, 0, notLimited, tableType, noDict, noDictIssue, acceleration);
        }
    } else {
        if (srcSize < LZ4_64Klimit) {
            const tableType_t tableType = byU16;
            LZ4_prepareTable(ctx, srcSize, tableType);
            if (ctx->currentOffset) {
                return LZ4_compress_generic(ctx, src, dst, srcSize, NULL, dstCapacity, limitedOutput, tableType, noDict, dictSmall, acceleration);
            } else {
                return LZ4_compress_generic(ctx, src, dst, srcSize, NULL, dstCapacity, limitedOutput, tableType, noDict, noDictIssue, acceleration);
            }
        } else {
            const tableType_t tableType = ((sizeof(void*)==4) && ((uptrval)src > LZ4_DISTANCE_MAX)) ? byPtr : byU32;
            LZ4_prepareTable(ctx, srcSize, tableType);
            return LZ4_compress_generic(ctx, src, dst, srcSize, NULL, dstCapacity, limitedOutput, tableType, noDict, noDictIssue, acceleration);
        }
    }
}


int LZ4_compress_fast(const char* source, char* dest, int inputSize, int maxOutputSize, int acceleration)
{
    int result;
#if (LZ4_HEAPMODE)
    LZ4_stream_t* ctxPtr = ALLOC(sizeof(LZ4_stream_t));   /* malloc-calloc always properly aligned */
    if (ctxPtr == NULL) return 0;
#else
    LZ4_stream_t ctx;
    LZ4_stream_t* const ctxPtr = &ctx;
#endif
    result = LZ4_compress_fast_extState(ctxPtr, source, dest, inputSize, maxOutputSize, acceleration);

#if (LZ4_HEAPMODE)
    FREEMEM(ctxPtr);
#endif
    return result;
}


int LZ4_compress_default(const char* src, char* dst, int srcSize, int maxOutputSize)
{
    return LZ4_compress_fast(src, dst, srcSize, maxOutputSize, 1);
}


/* hidden debug function */
/* strangely enough, gcc generates faster code when this function is uncommented, even if unused */
int LZ4_compress_fast_force(const char* src, char* dst, int srcSize, int dstCapacity, int acceleration)
{
    LZ4_stream_t ctx;
    LZ4_initStream(&ctx, sizeof(ctx));

    if (srcSize < LZ4_64Klimit) {
        return LZ4_compress_generic(&ctx.internal_donotuse, src, dst, srcSize, NULL, dstCapacity, limitedOutput, byU16,    noDict, noDictIssue, acceleration);
    } else {
        tableType_t const addrMode = (sizeof(void*) > 4) ? byU32 : byPtr;
        return LZ4_compress_generic(&ctx.internal_donotuse, src, dst, srcSize, NULL, dstCapacity, limitedOutput, addrMode, noDict, noDictIssue, acceleration);
    }
}


/* Note!: This function leaves the stream in an unclean/broken state!
 * It is not safe to subsequently use the same state with a _fastReset() or
 * _continue() call without resetting it. */
static int LZ4_compress_destSize_extState (LZ4_stream_t* state, const char* src, char* dst, int* srcSizePtr, int targetDstSize)
{
    void* const s = LZ4_initStream(state, sizeof (*state));
    assert(s != NULL); (void)s;

    if (targetDstSize >= LZ4_compressBound(*srcSizePtr)) {  /* compression success is guaranteed */
        return LZ4_compress_fast_extState(state, src, dst, *srcSizePtr, targetDstSize, 1);
    } else {
        if (*srcSizePtr < LZ4_64Klimit) {
            return LZ4_compress_generic(&state->internal_donotuse, src, dst, *srcSizePtr, srcSizePtr, targetDstSize, fillOutput, byU16, noDict, noDictIssue, 1);
        } else {
            tableType_t const addrMode = ((sizeof(void*)==4) && ((uptrval)src > LZ4_DISTANCE_MAX)) ? byPtr : byU32;
            return LZ4_compress_generic(&state->internal_donotuse, src, dst, *srcSizePtr, srcSizePtr, targetDstSize, fillOutput, addrMode, noDict, noDictIssue, 1);
    }   }
}


int LZ4_compress_destSize(const char* src, char* dst, int* srcSizePtr, int targetDstSize)
{
#if (LZ4_HEAPMODE)
    LZ4_stream_t* ctx = (LZ4_stream_t*)ALLOC(sizeof(LZ4_stream_t));   /* malloc-calloc always properly aligned */
    if (ctx == NULL) return 0;
#else
    LZ4_stream_t ctxBody;
    LZ4_stream_t* ctx = &ctxBody;
#endif

    int result = LZ4_compress_destSize_extState(ctx, src, dst, srcSizePtr, targetDstSize);

#if (LZ4_HEAPMODE)
    FREEMEM(ctx);
#endif
    return result;
}



/*-******************************
*  Streaming functions
********************************/

LZ4_stream_t* LZ4_createStream(void)
{
    LZ4_stream_t* const lz4s = (LZ4_stream_t*)ALLOC(sizeof(LZ4_stream_t));
    LZ4_STATIC_ASSERT(LZ4_STREAMSIZE >= sizeof(LZ4_stream_t_internal));    /* A compilation error here means LZ4_STREAMSIZE is not large enough */
    DEBUGLOG(4, "LZ4_createStream %p", lz4s);
    if (lz4s == NULL) return NULL;
    LZ4_initStream(lz4s, sizeof(*lz4s));
    return lz4s;
}

#ifndef _MSC_VER  /* for some reason, Visual fails the aligment test on 32-bit x86 :
                     it reports an aligment of 8-bytes,
                     while actually aligning LZ4_stream_t on 4 bytes. */
static size_t LZ4_stream_t_alignment(void)
{
    struct { char c; LZ4_stream_t t; } t_a;
    return sizeof(t_a) - sizeof(t_a.t);
}
#endif

LZ4_stream_t* LZ4_initStream (void* buffer, size_t size)
{
    DEBUGLOG(5, "LZ4_initStream");
    if (buffer == NULL) { return NULL; }
    if (size < sizeof(LZ4_stream_t)) { return NULL; }
#ifndef _MSC_VER  /* for some reason, Visual fails the aligment test on 32-bit x86 :
                     it reports an aligment of 8-bytes,
                     while actually aligning LZ4_stream_t on 4 bytes. */
    if (((size_t)buffer) & (LZ4_stream_t_alignment() - 1)) { return NULL; } /* alignment check */
#endif
    MEM_INIT(buffer, 0, sizeof(LZ4_stream_t));
    return (LZ4_stream_t*)buffer;
}

/* resetStream is now deprecated,
 * prefer initStream() which is more general */
void LZ4_resetStream (LZ4_stream_t* LZ4_stream)
{
    DEBUGLOG(5, "LZ4_resetStream (ctx:%p)", LZ4_stream);
    MEM_INIT(LZ4_stream, 0, sizeof(LZ4_stream_t));
}

void LZ4_resetStream_fast(LZ4_stream_t* ctx) {
    LZ4_prepareTable(&(ctx->internal_donotuse), 0, byU32);
}

int LZ4_freeStream (LZ4_stream_t* LZ4_stream)
{
    if (!LZ4_stream) return 0;   /* support free on NULL */
    DEBUGLOG(5, "LZ4_freeStream %p", LZ4_stream);
    FREEMEM(LZ4_stream);
    return (0);
}


#define HASH_UNIT sizeof(reg_t)
int LZ4_loadDict (LZ4_stream_t* LZ4_dict, const char* dictionary, int dictSize)
{
    LZ4_stream_t_internal* dict = &LZ4_dict->internal_donotuse;
    const tableType_t tableType = byU32;
    const BYTE* p = (const BYTE*)dictionary;
    const BYTE* const dictEnd = p + dictSize;
    const BYTE* base;

    DEBUGLOG(4, "LZ4_loadDict (%i bytes from %p into %p)", dictSize, dictionary, LZ4_dict);

    /* It's necessary to reset the context,
     * and not just continue it with prepareTable()
     * to avoid any risk of generating overflowing matchIndex
     * when compressing using this dictionary */
    LZ4_resetStream(LZ4_dict);

    /* We always increment the offset by 64 KB, since, if the dict is longer,
     * we truncate it to the last 64k, and if it's shorter, we still want to
     * advance by a whole window length so we can provide the guarantee that
     * there are only valid offsets in the window, which allows an optimization
     * in LZ4_compress_fast_continue() where it uses noDictIssue even when the
     * dictionary isn't a full 64k. */
    dict->currentOffset += 64 KB;

    if (dictSize < (int)HASH_UNIT) {
        return 0;
    }

    if ((dictEnd - p) > 64 KB) p = dictEnd - 64 KB;
    base = dictEnd - dict->currentOffset;
    dict->dictionary = p;
    dict->dictSize = (U32)(dictEnd - p);
    dict->tableType = tableType;

    while (p <= dictEnd-HASH_UNIT) {
        LZ4_putPosition(p, dict->hashTable, tableType, base);
        p+=3;
    }

    return (int)dict->dictSize;
}

void LZ4_attach_dictionary(LZ4_stream_t* workingStream, const LZ4_stream_t* dictionaryStream) {
    const LZ4_stream_t_internal* dictCtx = dictionaryStream == NULL ? NULL :
        &(dictionaryStream->internal_donotuse);

    DEBUGLOG(4, "LZ4_attach_dictionary (%p, %p, size %u)",
             workingStream, dictionaryStream,
             dictCtx != NULL ? dictCtx->dictSize : 0);

    /* Calling LZ4_resetStream_fast() here makes sure that changes will not be
     * erased by subsequent calls to LZ4_resetStream_fast() in case stream was
     * marked as having dirty context, e.g. requiring full reset.
     */
    LZ4_resetStream_fast(workingStream);

    if (dictCtx != NULL) {
        /* If the current offset is zero, we will never look in the
         * external dictionary context, since there is no value a table
         * entry can take that indicate a miss. In that case, we need
         * to bump the offset to something non-zero.
         */
        if (workingStream->internal_donotuse.currentOffset == 0) {
            workingStream->internal_donotuse.currentOffset = 64 KB;
        }

        /* Don't actually attach an empty dictionary.
         */
        if (dictCtx->dictSize == 0) {
            dictCtx = NULL;
        }
    }
    workingStream->internal_donotuse.dictCtx = dictCtx;
}


static void LZ4_renormDictT(LZ4_stream_t_internal* LZ4_dict, int nextSize)
{
    assert(nextSize >= 0);
    if (LZ4_dict->currentOffset + (unsigned)nextSize > 0x80000000) {   /* potential ptrdiff_t overflow (32-bits mode) */
        /* rescale hash table */
        U32 const delta = LZ4_dict->currentOffset - 64 KB;
        const BYTE* dictEnd = LZ4_dict->dictionary + LZ4_dict->dictSize;
        int i;
        DEBUGLOG(4, "LZ4_renormDictT");
        for (i=0; i<LZ4_HASH_SIZE_U32; i++) {
            if (LZ4_dict->hashTable[i] < delta) LZ4_dict->hashTable[i]=0;
            else LZ4_dict->hashTable[i] -= delta;
        }
        LZ4_dict->currentOffset = 64 KB;
        if (LZ4_dict->dictSize > 64 KB) LZ4_dict->dictSize = 64 KB;
        LZ4_dict->dictionary = dictEnd - LZ4_dict->dictSize;
    }
}


int LZ4_compress_fast_continue (LZ4_stream_t* LZ4_stream,
                                const char* source, char* dest,
                                int inputSize, int maxOutputSize,
                                int acceleration)
{
    const tableType_t tableType = byU32;
    LZ4_stream_t_internal* streamPtr = &LZ4_stream->internal_donotuse;
    const BYTE* dictEnd = streamPtr->dictionary + streamPtr->dictSize;

    DEBUGLOG(5, "LZ4_compress_fast_continue (inputSize=%i)", inputSize);

    if (streamPtr->dirty) { return 0; } /* Uninitialized structure detected */
    LZ4_renormDictT(streamPtr, inputSize);   /* avoid index overflow */
    if (acceleration < 1) acceleration = ACCELERATION_DEFAULT;

    /* invalidate tiny dictionaries */
    if ( (streamPtr->dictSize-1 < 4-1)   /* intentional underflow */
      && (dictEnd != (const BYTE*)source) ) {
        DEBUGLOG(5, "LZ4_compress_fast_continue: dictSize(%u) at addr:%p is too small", streamPtr->dictSize, streamPtr->dictionary);
        streamPtr->dictSize = 0;
        streamPtr->dictionary = (const BYTE*)source;
        dictEnd = (const BYTE*)source;
    }

    /* Check overlapping input/dictionary space */
    {   const BYTE* sourceEnd = (const BYTE*) source + inputSize;
        if ((sourceEnd > streamPtr->dictionary) && (sourceEnd < dictEnd)) {
            streamPtr->dictSize = (U32)(dictEnd - sourceEnd);
            if (streamPtr->dictSize > 64 KB) streamPtr->dictSize = 64 KB;
            if (streamPtr->dictSize < 4) streamPtr->dictSize = 0;
            streamPtr->dictionary = dictEnd - streamPtr->dictSize;
        }
    }

    /* prefix mode : source data follows dictionary */
    if (dictEnd == (const BYTE*)source) {
        if ((streamPtr->dictSize < 64 KB) && (streamPtr->dictSize < streamPtr->currentOffset))
            return LZ4_compress_generic(streamPtr, source, dest, inputSize, NULL, maxOutputSize, limitedOutput, tableType, withPrefix64k, dictSmall, acceleration);
        else
            return LZ4_compress_generic(streamPtr, source, dest, inputSize, NULL, maxOutputSize, limitedOutput, tableType, withPrefix64k, noDictIssue, acceleration);
    }

    /* external dictionary mode */
    {   int result;
        if (streamPtr->dictCtx) {
            /* We depend here on the fact that dictCtx'es (produced by
             * LZ4_loadDict) guarantee that their tables contain no references
             * to offsets between dictCtx->currentOffset - 64 KB and
             * dictCtx->currentOffset - dictCtx->dictSize. This makes it safe
             * to use noDictIssue even when the dict isn't a full 64 KB.
             */
            if (inputSize > 4 KB) {
                /* For compressing large blobs, it is faster to pay the setup
                 * cost to copy the dictionary's tables into the active context,
                 * so that the compression loop is only looking into one table.
                 */
                memcpy(streamPtr, streamPtr->dictCtx, sizeof(LZ4_stream_t));
                result = LZ4_compress_generic(streamPtr, source, dest, inputSize, NULL, maxOutputSize, limitedOutput, tableType, usingExtDict, noDictIssue, acceleration);
            } else {
                result = LZ4_compress_generic(streamPtr, source, dest, inputSize, NULL, maxOutputSize, limitedOutput, tableType, usingDictCtx, noDictIssue, acceleration);
            }
        } else {
            if ((streamPtr->dictSize < 64 KB) && (streamPtr->dictSize < streamPtr->currentOffset)) {
                result = LZ4_compress_generic(streamPtr, source, dest, inputSize, NULL, maxOutputSize, limitedOutput, tableType, usingExtDict, dictSmall, acceleration);
            } else {
                result = LZ4_compress_generic(streamPtr, source, dest, inputSize, NULL, maxOutputSize, limitedOutput, tableType, usingExtDict, noDictIssue, acceleration);
            }
        }
        streamPtr->dictionary = (const BYTE*)source;
        streamPtr->dictSize = (U32)inputSize;
        return result;
    }
}


/* Hidden debug function, to force-test external dictionary mode */
int LZ4_compress_forceExtDict (LZ4_stream_t* LZ4_dict, const char* source, char* dest, int srcSize)
{
    LZ4_stream_t_internal* streamPtr = &LZ4_dict->internal_donotuse;
    int result;

    LZ4_renormDictT(streamPtr, srcSize);

    if ((streamPtr->dictSize < 64 KB) && (streamPtr->dictSize < streamPtr->currentOffset)) {
        result = LZ4_compress_generic(streamPtr, source, dest, srcSize, NULL, 0, notLimited, byU32, usingExtDict, dictSmall, 1);
    } else {
        result = LZ4_compress_generic(streamPtr, source, dest, srcSize, NULL, 0, notLimited, byU32, usingExtDict, noDictIssue, 1);
    }

    streamPtr->dictionary = (const BYTE*)source;
    streamPtr->dictSize = (U32)srcSize;

    return result;
}


/*! LZ4_saveDict() :
 *  If previously compressed data block is not guaranteed to remain available at its memory location,
 *  save it into a safer place (char* safeBuffer).
 *  Note : you don't need to call LZ4_loadDict() afterwards,
 *         dictionary is immediately usable, you can therefore call LZ4_compress_fast_continue().
 *  Return : saved dictionary size in bytes (necessarily <= dictSize), or 0 if error.
 */
int LZ4_saveDict (LZ4_stream_t* LZ4_dict, char* safeBuffer, int dictSize)
{
    LZ4_stream_t_internal* const dict = &LZ4_dict->internal_donotuse;
    const BYTE* const previousDictEnd = dict->dictionary + dict->dictSize;

    if ((U32)dictSize > 64 KB) { dictSize = 64 KB; } /* useless to define a dictionary > 64 KB */
    if ((U32)dictSize > dict->dictSize) { dictSize = (int)dict->dictSize; }

    memmove(safeBuffer, previousDictEnd - dictSize, dictSize);

    dict->dictionary = (const BYTE*)safeBuffer;
    dict->dictSize = (U32)dictSize;

    return dictSize;
}



/*-*******************************
 *  Decompression functions
 ********************************/

typedef enum { endOnOutputSize = 0, endOnInputSize = 1 } endCondition_directive;
typedef enum { decode_full_block = 0, partial_decode = 1 } earlyEnd_directive;

#undef MIN
#define MIN(a,b)    ( (a) < (b) ? (a) : (b) )

/* Read the variable-length literal or match length.
 *
 * ip - pointer to use as input.
 * lencheck - end ip.  Return an error if ip advances >= lencheck.
 * loop_check - check ip >= lencheck in body of loop.  Returns loop_error if so.
 * initial_check - check ip >= lencheck before start of loop.  Returns initial_error if so.
 * error (output) - error code.  Should be set to 0 before call.
 */
// BEGIN EPIC MOD : The 'ok' identifier collides with symbols in the MacOS SDK
typedef enum { loop_error = -2, initial_error = -1, length_ok = 0 } variable_length_error;
// END EPIC MOD
LZ4_FORCE_INLINE unsigned
read_variable_length(const BYTE**ip, const BYTE* lencheck, int loop_check, int initial_check, variable_length_error* error)
{
  unsigned length = 0;
  unsigned s;
  if (initial_check && unlikely((*ip) >= lencheck)) {    /* overflow detection */
    *error = initial_error;
    return length;
  }
  do {
    s = **ip;
    (*ip)++;
    length += s;
    if (loop_check && unlikely((*ip) >= lencheck)) {    /* overflow detection */
      *error = loop_error;
      return length;
    }
  } while (s==255);

  return length;
}

/*! LZ4_decompress_generic() :
 *  This generic decompression function covers all use cases.
 *  It shall be instantiated several times, using different sets of directives.
 *  Note that it is important for performance that this function really get inlined,
 *  in order to remove useless branches during compilation optimization.
 */
LZ4_FORCE_INLINE int
LZ4_decompress_generic(
                 const char* const src,
                 char* const dst,
                 int srcSize,
                 int outputSize,         /* If endOnInput==endOnInputSize, this value is `dstCapacity` */

                 endCondition_directive endOnInput,   /* endOnOutputSize, endOnInputSize */
                 earlyEnd_directive partialDecoding,  /* full, partial */
                 dict_directive dict,                 /* noDict, withPrefix64k, usingExtDict */
                 const BYTE* const lowPrefix,  /* always <= dst, == dst when no prefix */
                 const BYTE* const dictStart,  /* only if dict==usingExtDict */
                 const size_t dictSize         /* note : = 0 if noDict */
                 )
{
    if (src == NULL) { return -1; }

    {   const BYTE* ip = (const BYTE*) src;
        const BYTE* const iend = ip + srcSize;

        BYTE* op = (BYTE*) dst;
        BYTE* const oend = op + outputSize;
        BYTE* cpy;

        const BYTE* const dictEnd = (dictStart == NULL) ? NULL : dictStart + dictSize;

        const int safeDecode = (endOnInput==endOnInputSize);
        const int checkOffset = ((safeDecode) && (dictSize < (int)(64 KB)));


        /* Set up the "end" pointers for the shortcut. */
        const BYTE* const shortiend = iend - (endOnInput ? 14 : 8) /*maxLL*/ - 2 /*offset*/;
        const BYTE* const shortoend = oend - (endOnInput ? 14 : 8) /*maxLL*/ - 18 /*maxML*/;

        const BYTE* match;
        size_t offset;
        unsigned token;
        size_t length;


        DEBUGLOG(5, "LZ4_decompress_generic (srcSize:%i, dstSize:%i)", srcSize, outputSize);

        /* Special cases */
        assert(lowPrefix <= op);
        if ((endOnInput) && (unlikely(outputSize==0))) {
            /* Empty output buffer */
            if (partialDecoding) return 0;
            return ((srcSize==1) && (*ip==0)) ? 0 : -1;
        }
        if ((!endOnInput) && (unlikely(outputSize==0))) { return (*ip==0 ? 1 : -1); }
        if ((endOnInput) && unlikely(srcSize==0)) { return -1; }

	/* Currently the fast loop shows a regression on qualcomm arm chips. */
#if LZ4_FAST_DEC_LOOP
        if ((oend - op) < FASTLOOP_SAFE_DISTANCE) {
            DEBUGLOG(6, "skip fast decode loop");
            goto safe_decode;
        }

        /* Fast loop : decode sequences as long as output < iend-FASTLOOP_SAFE_DISTANCE */
        while (1) {
            /* Main fastloop assertion: We can always wildcopy FASTLOOP_SAFE_DISTANCE */
            assert(oend - op >= FASTLOOP_SAFE_DISTANCE);
            if (endOnInput) { assert(ip < iend); }
            token = *ip++;
            length = token >> ML_BITS;  /* literal length */

            assert(!endOnInput || ip <= iend); /* ip < iend before the increment */

            /* decode literal length */
            if (length == RUN_MASK) {
// BEGIN EPIC MOD : The 'ok' identifier collides with symbols in the MacOS SDK
                variable_length_error error = length_ok;
// END EPIC MOD
                length += read_variable_length(&ip, iend-RUN_MASK, endOnInput, endOnInput, &error);
                if (error == initial_error) { goto _output_error; }
                if ((safeDecode) && unlikely((uptrval)(op)+length<(uptrval)(op))) { goto _output_error; } /* overflow detection */
                if ((safeDecode) && unlikely((uptrval)(ip)+length<(uptrval)(ip))) { goto _output_error; } /* overflow detection */

                /* copy literals */
                cpy = op+length;
                LZ4_STATIC_ASSERT(MFLIMIT >= WILDCOPYLENGTH);
                if (endOnInput) {  /* LZ4_decompress_safe() */
                    if ((cpy>oend-32) || (ip+length>iend-32)) { goto safe_literal_copy; }
                    LZ4_wildCopy32(op, ip, cpy);
                } else {   /* LZ4_decompress_fast() */
                    if (cpy>oend-8) { goto safe_literal_copy; }
                    LZ4_wildCopy8(op, ip, cpy); /* LZ4_decompress_fast() cannot copy more than 8 bytes at a time :
                                                 * it doesn't know input length, and only relies on end-of-block properties */
                }
                ip += length; op = cpy;
            } else {
                cpy = op+length;
                if (endOnInput) {  /* LZ4_decompress_safe() */
                    DEBUGLOG(7, "copy %u bytes in a 16-bytes stripe", (unsigned)length);
                    /* We don't need to check oend, since we check it once for each loop below */
                    if (ip > iend-(16 + 1/*max lit + offset + nextToken*/)) { goto safe_literal_copy; }
                    /* Literals can only be 14, but hope compilers optimize if we copy by a register size */
                    memcpy(op, ip, 16);
                } else {  /* LZ4_decompress_fast() */
                    /* LZ4_decompress_fast() cannot copy more than 8 bytes at a time :
                     * it doesn't know input length, and relies on end-of-block properties */
                    memcpy(op, ip, 8);
                    if (length > 8) { memcpy(op+8, ip+8, 8); }
                }
                ip += length; op = cpy;
            }

            /* get offset */
            offset = LZ4_readLE16(ip); ip+=2;
            match = op - offset;
            assert(match <= op);

            /* get matchlength */
            length = token & ML_MASK;

            if (length == ML_MASK) {
// BEGIN EPIC MOD : The 'ok' identifier collides with symbols in the MacOS SDK
              variable_length_error error = length_ok;
// END EPIC MOD
              if ((checkOffset) && (unlikely(match + dictSize < lowPrefix))) { goto _output_error; } /* Error : offset outside buffers */
              length += read_variable_length(&ip, iend - LASTLITERALS + 1, endOnInput, 0, &error);
// BEGIN EPIC MOD : The 'ok' identifier collides with symbols in the MacOS SDK
              if (error != length_ok) { goto _output_error; }
// END EPIC MOD
                if ((safeDecode) && unlikely((uptrval)(op)+length<(uptrval)op)) { goto _output_error; } /* overflow detection */
                length += MINMATCH;
                if (op + length >= oend - FASTLOOP_SAFE_DISTANCE) {
                    goto safe_match_copy;
                }
            } else {
                length += MINMATCH;
                if (op + length >= oend - FASTLOOP_SAFE_DISTANCE) {
                    goto safe_match_copy;
                }

                /* Fastpath check: Avoids a branch in LZ4_wildCopy32 if true */
                if ((dict == withPrefix64k) || (match >= lowPrefix)) {
                    if (offset >= 8) {
                        assert(match >= lowPrefix);
                        assert(match <= op);
                        assert(op + 18 <= oend);

                        memcpy(op, match, 8);
                        memcpy(op+8, match+8, 8);
                        memcpy(op+16, match+16, 2);
                        op += length;
                        continue;
            }   }   }

            if ((checkOffset) && (unlikely(match + dictSize < lowPrefix))) { goto _output_error; } /* Error : offset outside buffers */
            /* match starting within external dictionary */
            if ((dict==usingExtDict) && (match < lowPrefix)) {
                if (unlikely(op+length > oend-LASTLITERALS)) {
                    if (partialDecoding) {
                        length = MIN(length, (size_t)(oend-op));  /* reach end of buffer */
                    } else {
                        goto _output_error;  /* end-of-block condition violated */
                }   }

                if (length <= (size_t)(lowPrefix-match)) {
                    /* match fits entirely within external dictionary : just copy */
                    memmove(op, dictEnd - (lowPrefix-match), length);
                    op += length;
                } else {
                    /* match stretches into both external dictionary and current block */
                    size_t const copySize = (size_t)(lowPrefix - match);
                    size_t const restSize = length - copySize;
                    memcpy(op, dictEnd - copySize, copySize);
                    op += copySize;
                    if (restSize > (size_t)(op - lowPrefix)) {  /* overlap copy */
                        BYTE* const endOfMatch = op + restSize;
                        const BYTE* copyFrom = lowPrefix;
                        while (op < endOfMatch) { *op++ = *copyFrom++; }
                    } else {
                        memcpy(op, lowPrefix, restSize);
                        op += restSize;
                }   }
                continue;
            }

            /* copy match within block */
            cpy = op + length;

            assert((op <= oend) && (oend-op >= 32));
            if (unlikely(offset<16)) {
                LZ4_memcpy_using_offset(op, match, cpy, offset);
            } else {
                LZ4_wildCopy32(op, match, cpy);
            }

            op = cpy;   /* wildcopy correction */
        }
    safe_decode:
#endif

        /* Main Loop : decode remaining sequences where output < FASTLOOP_SAFE_DISTANCE */
        while (1) {
            token = *ip++;
            length = token >> ML_BITS;  /* literal length */

            assert(!endOnInput || ip <= iend); /* ip < iend before the increment */

            /* A two-stage shortcut for the most common case:
             * 1) If the literal length is 0..14, and there is enough space,
             * enter the shortcut and copy 16 bytes on behalf of the literals
             * (in the fast mode, only 8 bytes can be safely copied this way).
             * 2) Further if the match length is 4..18, copy 18 bytes in a similar
             * manner; but we ensure that there's enough space in the output for
             * those 18 bytes earlier, upon entering the shortcut (in other words,
             * there is a combined check for both stages).
             */
            if ( (endOnInput ? length != RUN_MASK : length <= 8)
                /* strictly "less than" on input, to re-enter the loop with at least one byte */
              && likely((endOnInput ? ip < shortiend : 1) & (op <= shortoend)) ) {
                /* Copy the literals */
                memcpy(op, ip, endOnInput ? 16 : 8);
                op += length; ip += length;

                /* The second stage: prepare for match copying, decode full info.
                 * If it doesn't work out, the info won't be wasted. */
                length = token & ML_MASK; /* match length */
                offset = LZ4_readLE16(ip); ip += 2;
                match = op - offset;
                assert(match <= op); /* check overflow */

                /* Do not deal with overlapping matches. */
                if ( (length != ML_MASK)
                  && (offset >= 8)
                  && (dict==withPrefix64k || match >= lowPrefix) ) {
                    /* Copy the match. */
                    memcpy(op + 0, match + 0, 8);
                    memcpy(op + 8, match + 8, 8);
                    memcpy(op +16, match +16, 2);
                    op += length + MINMATCH;
                    /* Both stages worked, load the next token. */
                    continue;
                }

                /* The second stage didn't work out, but the info is ready.
                 * Propel it right to the point of match copying. */
                goto _copy_match;
            }

            /* decode literal length */
            if (length == RUN_MASK) {
// BEGIN EPIC MOD : The 'ok' identifier collides with symbols in the MacOS SDK
                variable_length_error error = length_ok;
// END EPIC MOD
                length += read_variable_length(&ip, iend-RUN_MASK, endOnInput, endOnInput, &error);
                if (error == initial_error) { goto _output_error; }
                if ((safeDecode) && unlikely((uptrval)(op)+length<(uptrval)(op))) { goto _output_error; } /* overflow detection */
                if ((safeDecode) && unlikely((uptrval)(ip)+length<(uptrval)(ip))) { goto _output_error; } /* overflow detection */
            }

            /* copy literals */
            cpy = op+length;
#if LZ4_FAST_DEC_LOOP
        safe_literal_copy:
#endif
            LZ4_STATIC_ASSERT(MFLIMIT >= WILDCOPYLENGTH);
            if ( ((endOnInput) && ((cpy>oend-MFLIMIT) || (ip+length>iend-(2+1+LASTLITERALS))) )
              || ((!endOnInput) && (cpy>oend-WILDCOPYLENGTH)) )
            {
                /* We've either hit the input parsing restriction or the output parsing restriction.
                 * If we've hit the input parsing condition then this must be the last sequence.
                 * If we've hit the output parsing condition then we are either using partialDecoding
                 * or we've hit the output parsing condition.
                 */
                if (partialDecoding) {
                    /* Since we are partial decoding we may be in this block because of the output parsing
                     * restriction, which is not valid since the output buffer is allowed to be undersized.
                     */
                    assert(endOnInput);
                    /* If we're in this block because of the input parsing condition, then we must be on the
                     * last sequence (or invalid), so we must check that we exactly consume the input.
                     */
                    if ((ip+length>iend-(2+1+LASTLITERALS)) && (ip+length != iend)) { goto _output_error; }
                    assert(ip+length <= iend);
                    /* We are finishing in the middle of a literals segment.
                     * Break after the copy.
                     */
                    if (cpy > oend) {
                        cpy = oend;
                        assert(op<=oend);
                        length = (size_t)(oend-op);
                    }
                    assert(ip+length <= iend);
                } else {
                    /* We must be on the last sequence because of the parsing limitations so check
                     * that we exactly regenerate the original size (must be exact when !endOnInput).
                     */
                    if ((!endOnInput) && (cpy != oend)) { goto _output_error; }
                     /* We must be on the last sequence (or invalid) because of the parsing limitations
                      * so check that we exactly consume the input and don't overrun the output buffer.
                      */
                    if ((endOnInput) && ((ip+length != iend) || (cpy > oend))) { goto _output_error; }
                }
                memmove(op, ip, length);  /* supports overlapping memory regions, which only matters for in-place decompression scenarios */
                ip += length;
                op += length;
                /* Necessarily EOF when !partialDecoding. When partialDecoding
                 * it is EOF if we've either filled the output buffer or hit
                 * the input parsing restriction.
                 */
                if (!partialDecoding || (cpy == oend) || (ip == iend)) {
                    break;
                }
            } else {
                LZ4_wildCopy8(op, ip, cpy);   /* may overwrite up to WILDCOPYLENGTH beyond cpy */
                ip += length; op = cpy;
            }

            /* get offset */
            offset = LZ4_readLE16(ip); ip+=2;
            match = op - offset;

            /* get matchlength */
            length = token & ML_MASK;

    _copy_match:
            if (length == ML_MASK) {
// BEGIN EPIC MOD : The 'ok' identifier collides with symbols in the MacOS SDK
              variable_length_error error = length_ok;
// END EPIC MOD
              length += read_variable_length(&ip, iend - LASTLITERALS + 1, endOnInput, 0, &error);
// BEGIN EPIC MOD : The 'ok' identifier collides with symbols in the MacOS SDK
              if (error != length_ok) goto _output_error;
// END EPIC MOD
                if ((safeDecode) && unlikely((uptrval)(op)+length<(uptrval)op)) goto _output_error;   /* overflow detection */
            }
            length += MINMATCH;

#if LZ4_FAST_DEC_LOOP
        safe_match_copy:
#endif
            if ((checkOffset) && (unlikely(match + dictSize < lowPrefix))) goto _output_error;   /* Error : offset outside buffers */
            /* match starting within external dictionary */
            if ((dict==usingExtDict) && (match < lowPrefix)) {
                if (unlikely(op+length > oend-LASTLITERALS)) {
                    if (partialDecoding) length = MIN(length, (size_t)(oend-op));
                    else goto _output_error;   /* doesn't respect parsing restriction */
                }

                if (length <= (size_t)(lowPrefix-match)) {
                    /* match fits entirely within external dictionary : just copy */
                    memmove(op, dictEnd - (lowPrefix-match), length);
                    op += length;
                } else {
                    /* match stretches into both external dictionary and current block */
                    size_t const copySize = (size_t)(lowPrefix - match);
                    size_t const restSize = length - copySize;
                    memcpy(op, dictEnd - copySize, copySize);
                    op += copySize;
                    if (restSize > (size_t)(op - lowPrefix)) {  /* overlap copy */
                        BYTE* const endOfMatch = op + restSize;
                        const BYTE* copyFrom = lowPrefix;
                        while (op < endOfMatch) *op++ = *copyFrom++;
                    } else {
                        memcpy(op, lowPrefix, restSize);
                        op += restSize;
                }   }
                continue;
            }
            assert(match >= lowPrefix);

            /* copy match within block */
            cpy = op + length;

            /* partialDecoding : may end anywhere within the block */
            assert(op<=oend);
            if (partialDecoding && (cpy > oend-MATCH_SAFEGUARD_DISTANCE)) {
                size_t const mlen = MIN(length, (size_t)(oend-op));
                const BYTE* const matchEnd = match + mlen;
                BYTE* const copyEnd = op + mlen;
                if (matchEnd > op) {   /* overlap copy */
                    while (op < copyEnd) { *op++ = *match++; }
                } else {
                    memcpy(op, match, mlen);
                }
                op = copyEnd;
                if (op == oend) { break; }
                continue;
            }

            if (unlikely(offset<8)) {
                LZ4_write32(op, 0);   /* silence msan warning when offset==0 */
                op[0] = match[0];
                op[1] = match[1];
                op[2] = match[2];
                op[3] = match[3];
                match += inc32table[offset];
                memcpy(op+4, match, 4);
                match -= dec64table[offset];
            } else {
                memcpy(op, match, 8);
                match += 8;
            }
            op += 8;

            if (unlikely(cpy > oend-MATCH_SAFEGUARD_DISTANCE)) {
                BYTE* const oCopyLimit = oend - (WILDCOPYLENGTH-1);
                if (cpy > oend-LASTLITERALS) { goto _output_error; } /* Error : last LASTLITERALS bytes must be literals (uncompressed) */
                if (op < oCopyLimit) {
                    LZ4_wildCopy8(op, match, oCopyLimit);
                    match += oCopyLimit - op;
                    op = oCopyLimit;
                }
                while (op < cpy) { *op++ = *match++; }
            } else {
                memcpy(op, match, 8);
                if (length > 16)  { LZ4_wildCopy8(op+8, match+8, cpy); }
            }
            op = cpy;   /* wildcopy correction */
        }

        /* end of decoding */
        if (endOnInput) {
           return (int) (((char*)op)-dst);     /* Nb of output bytes decoded */
       } else {
           return (int) (((const char*)ip)-src);   /* Nb of input bytes read */
       }

        /* Overflow error detected */
    _output_error:
        return (int) (-(((const char*)ip)-src))-1;
    }
}


/*===== Instantiate the API decoding functions. =====*/

LZ4_FORCE_O2_GCC_PPC64LE
int LZ4_decompress_safe(const char* source, char* dest, int compressedSize, int maxDecompressedSize)
{
    return LZ4_decompress_generic(source, dest, compressedSize, maxDecompressedSize,
                                  endOnInputSize, decode_full_block, noDict,
                                  (BYTE*)dest, NULL, 0);
}

LZ4_FORCE_O2_GCC_PPC64LE
int LZ4_decompress_safe_partial(const char* src, char* dst, int compressedSize, int targetOutputSize, int dstCapacity)
{
    dstCapacity = MIN(targetOutputSize, dstCapacity);
    return LZ4_decompress_generic(src, dst, compressedSize, dstCapacity,
                                  endOnInputSize, partial_decode,
                                  noDict, (BYTE*)dst, NULL, 0);
}

LZ4_FORCE_O2_GCC_PPC64LE
int LZ4_decompress_fast(const char* source, char* dest, int originalSize)
{
    return LZ4_decompress_generic(source, dest, 0, originalSize,
                                  endOnOutputSize, decode_full_block, withPrefix64k,
                                  (BYTE*)dest - 64 KB, NULL, 0);
}

/*===== Instantiate a few more decoding cases, used more than once. =====*/

LZ4_FORCE_O2_GCC_PPC64LE /* Exported, an obsolete API function. */
int LZ4_decompress_safe_withPrefix64k(const char* source, char* dest, int compressedSize, int maxOutputSize)
{
    return LZ4_decompress_generic(source, dest, compressedSize, maxOutputSize,
                                  endOnInputSize, decode_full_block, withPrefix64k,
                                  (BYTE*)dest - 64 KB, NULL, 0);
}

/* Another obsolete API function, paired with the previous one. */
int LZ4_decompress_fast_withPrefix64k(const char* source, char* dest, int originalSize)
{
    /* LZ4_decompress_fast doesn't validate match offsets,
     * and thus serves well with any prefixed dictionary. */
    return LZ4_decompress_fast(source, dest, originalSize);
}

LZ4_FORCE_O2_GCC_PPC64LE
static int LZ4_decompress_safe_withSmallPrefix(const char* source, char* dest, int compressedSize, int maxOutputSize,
                                               size_t prefixSize)
{
    return LZ4_decompress_generic(source, dest, compressedSize, maxOutputSize,
                                  endOnInputSize, decode_full_block, noDict,
                                  (BYTE*)dest-prefixSize, NULL, 0);
}

LZ4_FORCE_O2_GCC_PPC64LE
int LZ4_decompress_safe_forceExtDict(const char* source, char* dest,
                                     int compressedSize, int maxOutputSize,
                                     const void* dictStart, size_t dictSize)
{
    return LZ4_decompress_generic(source, dest, compressedSize, maxOutputSize,
                                  endOnInputSize, decode_full_block, usingExtDict,
                                  (BYTE*)dest, (const BYTE*)dictStart, dictSize);
}

LZ4_FORCE_O2_GCC_PPC64LE
static int LZ4_decompress_fast_extDict(const char* source, char* dest, int originalSize,
                                       const void* dictStart, size_t dictSize)
{
    return LZ4_decompress_generic(source, dest, 0, originalSize,
                                  endOnOutputSize, decode_full_block, usingExtDict,
                                  (BYTE*)dest, (const BYTE*)dictStart, dictSize);
}

/* The "double dictionary" mode, for use with e.g. ring buffers: the first part
 * of the dictionary is passed as prefix, and the second via dictStart + dictSize.
 * These routines are used only once, in LZ4_decompress_*_continue().
 */
LZ4_FORCE_INLINE
int LZ4_decompress_safe_doubleDict(const char* source, char* dest, int compressedSize, int maxOutputSize,
                                   size_t prefixSize, const void* dictStart, size_t dictSize)
{
    return LZ4_decompress_generic(source, dest, compressedSize, maxOutputSize,
                                  endOnInputSize, decode_full_block, usingExtDict,
                                  (BYTE*)dest-prefixSize, (const BYTE*)dictStart, dictSize);
}

LZ4_FORCE_INLINE
int LZ4_decompress_fast_doubleDict(const char* source, char* dest, int originalSize,
                                   size_t prefixSize, const void* dictStart, size_t dictSize)
{
    return LZ4_decompress_generic(source, dest, 0, originalSize,
                                  endOnOutputSize, decode_full_block, usingExtDict,
                                  (BYTE*)dest-prefixSize, (const BYTE*)dictStart, dictSize);
}

/*===== streaming decompression functions =====*/

LZ4_streamDecode_t* LZ4_createStreamDecode(void)
{
    LZ4_streamDecode_t* lz4s = (LZ4_streamDecode_t*) ALLOC_AND_ZERO(sizeof(LZ4_streamDecode_t));
    LZ4_STATIC_ASSERT(LZ4_STREAMDECODESIZE >= sizeof(LZ4_streamDecode_t_internal));    /* A compilation error here means LZ4_STREAMDECODESIZE is not large enough */
    return lz4s;
}

int LZ4_freeStreamDecode (LZ4_streamDecode_t* LZ4_stream)
{
    if (LZ4_stream == NULL) { return 0; }  /* support free on NULL */
    FREEMEM(LZ4_stream);
    return 0;
}

/*! LZ4_setStreamDecode() :
 *  Use this function to instruct where to find the dictionary.
 *  This function is not necessary if previous data is still available where it was decoded.
 *  Loading a size of 0 is allowed (same effect as no dictionary).
 * @return : 1 if OK, 0 if error
 */
int LZ4_setStreamDecode (LZ4_streamDecode_t* LZ4_streamDecode, const char* dictionary, int dictSize)
{
    LZ4_streamDecode_t_internal* lz4sd = &LZ4_streamDecode->internal_donotuse;
    lz4sd->prefixSize = (size_t) dictSize;
    lz4sd->prefixEnd = (const BYTE*) dictionary + dictSize;
    lz4sd->externalDict = NULL;
    lz4sd->extDictSize  = 0;
    return 1;
}

/*! LZ4_decoderRingBufferSize() :
 *  when setting a ring buffer for streaming decompression (optional scenario),
 *  provides the minimum size of this ring buffer
 *  to be compatible with any source respecting maxBlockSize condition.
 *  Note : in a ring buffer scenario,
 *  blocks are presumed decompressed next to each other.
 *  When not enough space remains for next block (remainingSize < maxBlockSize),
 *  decoding resumes from beginning of ring buffer.
 * @return : minimum ring buffer size,
 *           or 0 if there is an error (invalid maxBlockSize).
 */
int LZ4_decoderRingBufferSize(int maxBlockSize)
{
    if (maxBlockSize < 0) return 0;
    if (maxBlockSize > LZ4_MAX_INPUT_SIZE) return 0;
    if (maxBlockSize < 16) maxBlockSize = 16;
    return LZ4_DECODER_RING_BUFFER_SIZE(maxBlockSize);
}

/*
*_continue() :
    These decoding functions allow decompression of multiple blocks in "streaming" mode.
    Previously decoded blocks must still be available at the memory position where they were decoded.
    If it's not possible, save the relevant part of decoded data into a safe buffer,
    and indicate where it stands using LZ4_setStreamDecode()
*/
LZ4_FORCE_O2_GCC_PPC64LE
int LZ4_decompress_safe_continue (LZ4_streamDecode_t* LZ4_streamDecode, const char* source, char* dest, int compressedSize, int maxOutputSize)
{
    LZ4_streamDecode_t_internal* lz4sd = &LZ4_streamDecode->internal_donotuse;
    int result;

    if (lz4sd->prefixSize == 0) {
        /* The first call, no dictionary yet. */
        assert(lz4sd->extDictSize == 0);
        result = LZ4_decompress_safe(source, dest, compressedSize, maxOutputSize);
        if (result <= 0) return result;
        lz4sd->prefixSize = (size_t)result;
        lz4sd->prefixEnd = (BYTE*)dest + result;
    } else if (lz4sd->prefixEnd == (BYTE*)dest) {
        /* They're rolling the current segment. */
        if (lz4sd->prefixSize >= 64 KB - 1)
            result = LZ4_decompress_safe_withPrefix64k(source, dest, compressedSize, maxOutputSize);
        else if (lz4sd->extDictSize == 0)
            result = LZ4_decompress_safe_withSmallPrefix(source, dest, compressedSize, maxOutputSize,
                                                         lz4sd->prefixSize);
        else
            result = LZ4_decompress_safe_doubleDict(source, dest, compressedSize, maxOutputSize,
                                                    lz4sd->prefixSize, lz4sd->externalDict, lz4sd->extDictSize);
        if (result <= 0) return result;
        lz4sd->prefixSize += (size_t)result;
        lz4sd->prefixEnd  += result;
    } else {
        /* The buffer wraps around, or they're switching to another buffer. */
        lz4sd->extDictSize = lz4sd->prefixSize;
        lz4sd->externalDict = lz4sd->prefixEnd - lz4sd->extDictSize;
        result = LZ4_decompress_safe_forceExtDict(source, dest, compressedSize, maxOutputSize,
                                                  lz4sd->externalDict, lz4sd->extDictSize);
        if (result <= 0) return result;
        lz4sd->prefixSize = (size_t)result;
        lz4sd->prefixEnd  = (BYTE*)dest + result;
    }

    return result;
}

LZ4_FORCE_O2_GCC_PPC64LE
int LZ4_decompress_fast_continue (LZ4_streamDecode_t* LZ4_streamDecode, const char* source, char* dest, int originalSize)
{
    LZ4_streamDecode_t_internal* lz4sd = &LZ4_streamDecode->internal_donotuse;
    int result;
    assert(originalSize >= 0);

    if (lz4sd->prefixSize == 0) {
        assert(lz4sd->extDictSize == 0);
        result = LZ4_decompress_fast(source, dest, originalSize);
        if (result <= 0) return result;
        lz4sd->prefixSize = (size_t)originalSize;
        lz4sd->prefixEnd = (BYTE*)dest + originalSize;
    } else if (lz4sd->prefixEnd == (BYTE*)dest) {
        if (lz4sd->prefixSize >= 64 KB - 1 || lz4sd->extDictSize == 0)
            result = LZ4_decompress_fast(source, dest, originalSize);
        else
            result = LZ4_decompress_fast_doubleDict(source, dest, originalSize,
                                                    lz4sd->prefixSize, lz4sd->externalDict, lz4sd->extDictSize);
        if (result <= 0) return result;
        lz4sd->prefixSize += (size_t)originalSize;
        lz4sd->prefixEnd  += originalSize;
    } else {
        lz4sd->extDictSize = lz4sd->prefixSize;
        lz4sd->externalDict = lz4sd->prefixEnd - lz4sd->extDictSize;
        result = LZ4_decompress_fast_extDict(source, dest, originalSize,
                                             lz4sd->externalDict, lz4sd->extDictSize);
        if (result <= 0) return result;
        lz4sd->prefixSize = (size_t)originalSize;
        lz4sd->prefixEnd  = (BYTE*)dest + originalSize;
    }

    return result;
}


/*
Advanced decoding functions :
*_usingDict() :
    These decoding functions work the same as "_continue" ones,
    the dictionary must be explicitly provided within parameters
*/

int LZ4_decompress_safe_usingDict(const char* source, char* dest, int compressedSize, int maxOutputSize, const char* dictStart, int dictSize)
{
    if (dictSize==0)
        return LZ4_decompress_safe(source, dest, compressedSize, maxOutputSize);
    if (dictStart+dictSize == dest) {
        if (dictSize >= 64 KB - 1) {
            return LZ4_decompress_safe_withPrefix64k(source, dest, compressedSize, maxOutputSize);
        }
        assert(dictSize >= 0);
        return LZ4_decompress_safe_withSmallPrefix(source, dest, compressedSize, maxOutputSize, (size_t)dictSize);
    }
    assert(dictSize >= 0);
    return LZ4_decompress_safe_forceExtDict(source, dest, compressedSize, maxOutputSize, dictStart, (size_t)dictSize);
}

int LZ4_decompress_fast_usingDict(const char* source, char* dest, int originalSize, const char* dictStart, int dictSize)
{
    if (dictSize==0 || dictStart+dictSize == dest)
        return LZ4_decompress_fast(source, dest, originalSize);
    assert(dictSize >= 0);
    return LZ4_decompress_fast_extDict(source, dest, originalSize, dictStart, (size_t)dictSize);
}


/*=*************************************************
*  Obsolete Functions
***************************************************/
/* obsolete compression functions */
int LZ4_compress_limitedOutput(const char* source, char* dest, int inputSize, int maxOutputSize)
{
    return LZ4_compress_default(source, dest, inputSize, maxOutputSize);
}
int LZ4_compress(const char* src, char* dest, int srcSize)
{
    return LZ4_compress_default(src, dest, srcSize, LZ4_compressBound(srcSize));
}
int LZ4_compress_limitedOutput_withState (void* state, const char* src, char* dst, int srcSize, int dstSize)
{
    return LZ4_compress_fast_extState(state, src, dst, srcSize, dstSize, 1);
}
int LZ4_compress_withState (void* state, const char* src, char* dst, int srcSize)
{
    return LZ4_compress_fast_extState(state, src, dst, srcSize, LZ4_compressBound(srcSize), 1);
}
int LZ4_compress_limitedOutput_continue (LZ4_stream_t* LZ4_stream, const char* src, char* dst, int srcSize, int dstCapacity)
{
    return LZ4_compress_fast_continue(LZ4_stream, src, dst, srcSize, dstCapacity, 1);
}
int LZ4_compress_continue (LZ4_stream_t* LZ4_stream, const char* source, char* dest, int inputSize)
{
    return LZ4_compress_fast_continue(LZ4_stream, source, dest, inputSize, LZ4_compressBound(inputSize), 1);
}

/*
These decompression functions are deprecated and should no longer be used.
They are only provided here for compatibility with older user programs.
- LZ4_uncompress is totally equivalent to LZ4_decompress_fast
- LZ4_uncompress_unknownOutputSize is totally equivalent to LZ4_decompress_safe
*/
int LZ4_uncompress (const char* source, char* dest, int outputSize)
{
    return LZ4_decompress_fast(source, dest, outputSize);
}
int LZ4_uncompress_unknownOutputSize (const char* source, char* dest, int isize, int maxOutputSize)
{
    return LZ4_decompress_safe(source, dest, isize, maxOutputSize);
}

/* Obsolete Streaming functions */

int LZ4_sizeofStreamState() { return LZ4_STREAMSIZE; }

int LZ4_resetStreamState(void* state, char* inputBuffer)
{
    (void)inputBuffer;
    LZ4_resetStream((LZ4_stream_t*)state);
    return 0;
}

void* LZ4_create (char* inputBuffer)
{
    (void)inputBuffer;
    return LZ4_createStream();
}

char* LZ4_slideInputBuffer (void* state)
{
    /* avoid const char * -> char * conversion warning */
    return (char *)(uptrval)((LZ4_stream_t*)state)->internal_donotuse.dictionary;
}

#endif   /* LZ4_COMMONDEFS_ONLY */

LZ4_END_NAMESPACE // EPIC MOD : Wrap library in an optional namespace
