/*
 *  LZ4 - Fast LZ compression algorithm
 *  Header File
 *  Copyright (C) 2011-present, Yann Collet.

   BSD 2-Clause License (http://www.opensource.org/licenses/bsd-license.php)

   Redistribution and use in source and binary forms, with or without
   modification, are permitted provided that the following conditions are
   met:

       * Redistributions of source code must retain the above copyright
   notice, this list of conditions and the following disclaimer.
       * Redistributions in binary form must reproduce the above
   copyright notice, this list of conditions and the following disclaimer
   in the documentation and/or other materials provided with the
   distribution.

   THIS SOFTWARE IS PROVIDED BY THE COPYRIGHT HOLDERS AND CONTRIBUTORS
   "AS IS" AND ANY EXPRESS OR IMPLIED WARRANTIES, INCLUDING, BUT NOT
   LIMITED TO, THE IMPLIED WARRANTIES OF MERCHANTABILITY AND FITNESS FOR
   A PARTICULAR PURPOSE ARE DISCLAIMED. IN NO EVENT SHALL THE COPYRIGHT
   OWNER OR CONTRIBUTORS BE LIABLE FOR ANY DIRECT, INDIRECT, INCIDENTAL,
   SPECIAL, EXEMPLARY, OR CONSEQUENTIAL DAMAGES (INCLUDING, BUT NOT
   LIMITED TO, PROCUREMENT OF SUBSTITUTE GOODS OR SERVICES; LOSS OF USE,
   DATA, OR PROFITS; OR BUSINESS INTERRUPTION) HOWEVER CAUSED AND ON ANY
   THEORY OF LIABILITY, WHETHER IN CONTRACT, STRICT LIABILITY, OR TORT
   (INCLUDING NEGLIGENCE OR OTHERWISE) ARISING IN ANY WAY OUT OF THE USE
   OF THIS SOFTWARE, EVEN IF ADVISED OF THE POSSIBILITY OF SUCH DAMAGE.

   You can contact the author at :
    - LZ4 homepage : http://www.lz4.org
    - LZ4 source repository : https://github.com/lz4/lz4
*/

#pragma once // EPIC MOD: Placate IncludeTool warning

#if defined (__cplusplus)
extern "C" {
#endif

// BEGIN EPIC MOD : Wrap library in an optional namespace
#if defined(LZ4_NAMESPACE) && defined(__cplusplus)
#  define LZ4_BEGIN_NAMESPACE   extern "C++" { namespace LZ4_NAMESPACE {
#  define LZ4_END_NAMESPACE     } /* extern */ } /* namespace */
#else
#  define LZ4_BEGIN_NAMESPACE
#  define LZ4_END_NAMESPACE
#endif // LZ4_NAMESPACE
// END EPIC MOD

#ifndef LZ4_H_2983827168210
#define LZ4_H_2983827168210

/* --- Dependency --- */
#include <stddef.h>   /* size_t */


/**
  Introduction

  LZ4 is lossless compression algorithm, providing compression speed >500 MB/s per core,
  scalable with multi-cores CPU. It features an extremely fast decoder, with speed in
  multiple GB/s per core, typically reaching RAM speed limits on multi-core systems.

  The LZ4 compression library provides in-memory compression and decompression functions.
  It gives full buffer control to user.
  Compression can be done in:
    - a single step (described as Simple Functions)
    - a single step, reusing a context (described in Advanced Functions)
    - unbounded multiple steps (described as Streaming compression)

  lz4.h generates and decodes LZ4-compressed blocks (doc/lz4_Block_format.md).
  Decompressing such a compressed block requires additional metadata.
  Exact metadata depends on exact decompression function.
  For the typical case of LZ4_decompress_safe(),
  metadata includes block's compressed size, and maximum bound of decompressed size.
  Each application is free to encode and pass such metadata in whichever way it wants.

  lz4.h only handle blocks, it can not generate Frames.

  Blocks are different from Frames (doc/lz4_Frame_format.md).
  Frames bundle both blocks and metadata in a specified manner.
  Embedding metadata is required for compressed data to be self-contained and portable.
  Frame format is delivered through a companion API, declared in lz4frame.h.
  The `lz4` CLI can only manage frames.
*/

/*^***************************************************************
*  Export parameters
*****************************************************************/
/*
*  LZ4_DLL_EXPORT :
*  Enable exporting of functions when building a Windows DLL
*  LZ4LIB_VISIBILITY :
*  Control library symbols visibility.
*/
#ifndef LZ4LIB_VISIBILITY
#  if defined(__GNUC__) && (__GNUC__ >= 4)
#    define LZ4LIB_VISIBILITY __attribute__ ((visibility ("default")))
#  else
#    define LZ4LIB_VISIBILITY
#  endif
#endif
#if defined(LZ4_DLL_EXPORT) && (LZ4_DLL_EXPORT==1)
#  define LZ4LIB_API __declspec(dllexport) LZ4LIB_VISIBILITY
#elif defined(LZ4_DLL_IMPORT) && (LZ4_DLL_IMPORT==1)
#  define LZ4LIB_API __declspec(dllimport) LZ4LIB_VISIBILITY /* It isn't required but allows to generate better code, saving a function pointer load from the IAT and an indirect jump.*/
#else
#  define LZ4LIB_API LZ4LIB_VISIBILITY
#endif

/*------   Version   ------*/
#define LZ4_VERSION_MAJOR    1    /* for breaking interface changes  */
#define LZ4_VERSION_MINOR    9    /* for new (non-breaking) interface capabilities */
#define LZ4_VERSION_RELEASE  2    /* for tweaks, bug-fixes, or development */

#define LZ4_VERSION_NUMBER (LZ4_VERSION_MAJOR *100*100 + LZ4_VERSION_MINOR *100 + LZ4_VERSION_RELEASE)

#define LZ4_LIB_VERSION LZ4_VERSION_MAJOR.LZ4_VERSION_MINOR.LZ4_VERSION_RELEASE
#define LZ4_QUOTE(str) #str
#define LZ4_EXPAND_AND_QUOTE(str) LZ4_QUOTE(str)
#define LZ4_VERSION_STRING LZ4_EXPAND_AND_QUOTE(LZ4_LIB_VERSION)

LZ4_BEGIN_NAMESPACE // EPIC MOD : Wrap library in an optional namespace

LZ4LIB_API int LZ4_versionNumber (void);  /**< library version number; useful to check dll version */
LZ4LIB_API const char* LZ4_versionString (void);   /**< library version string; useful to check dll version */


/*-************************************
*  Tuning parameter
**************************************/
/*!
 * LZ4_MEMORY_USAGE :
 * Memory usage formula : N->2^N Bytes (examples : 10 -> 1KB; 12 -> 4KB ; 16 -> 64KB; 20 -> 1MB; etc.)
 * Increasing memory usage improves compression ratio.
 * Reduced memory usage may improve speed, thanks to better cache locality.
 * Default value is 14, for 16KB, which nicely fits into Intel x86 L1 cache
 */
#ifndef LZ4_MEMORY_USAGE
# define LZ4_MEMORY_USAGE 14
#endif


/*-************************************
*  Simple Functions
**************************************/
/*! LZ4_compress_default() :
 *  Compresses 'srcSize' bytes from buffer 'src'
 *  into already allocated 'dst' buffer of size 'dstCapacity'.
 *  Compression is guaranteed to succeed if 'dstCapacity' >= LZ4_compressBound(srcSize).
 *  It also runs faster, so it's a recommended setting.
 *  If the function cannot compress 'src' into a more limited 'dst' budget,
 *  compression stops *immediately*, and the function result is zero.
 *  In which case, 'dst' content is undefined (invalid).
 *      srcSize : max supported value is LZ4_MAX_INPUT_SIZE.
 *      dstCapacity : size of buffer 'dst' (which must be already allocated)
 *     @return  : the number of bytes written into buffer 'dst' (necessarily <= dstCapacity)
 *                or 0 if compression fails
 * Note : This function is protected against buffer overflow scenarios (never writes outside 'dst' buffer, nor read outside 'source' buffer).
 */
LZ4LIB_API int LZ4_compress_default(const char* src, char* dst, int srcSize, int dstCapacity);

/*! LZ4_decompress_safe() :
 *  compressedSize : is the exact complete size of the compressed block.
 *  dstCapacity : is the size of destination buffer (which must be already allocated), presumed an upper bound of decompressed size.
 * @return : the number of bytes decompressed into destination buffer (necessarily <= dstCapacity)
 *           If destination buffer is not large enough, decoding will stop and output an error code (negative value).
 *           If the source stream is detected malformed, the function will stop decoding and return a negative result.
 * Note 1 : This function is protected against malicious data packets :
 *          it will never writes outside 'dst' buffer, nor read outside 'source' buffer,
 *          even if the compressed block is maliciously modified to order the decoder to do these actions.
 *          In such case, the decoder stops immediately, and considers the compressed block malformed.
 * Note 2 : compressedSize and dstCapacity must be provided to the function, the compressed block does not contain them.
 *          The implementation is free to send / store / derive this information in whichever way is most beneficial.
 *          If there is a need for a different format which bundles together both compressed data and its metadata, consider looking at lz4frame.h instead.
 */
LZ4LIB_API int LZ4_decompress_safe (const char* src, char* dst, int compressedSize, int dstCapacity);


/*-************************************
*  Advanced Functions
**************************************/
#define LZ4_MAX_INPUT_SIZE        0x7E000000   /* 2 113 929 216 bytes */
#define LZ4_COMPRESSBOUND(isize)  ((unsigned)(isize) > (unsigned)LZ4_MAX_INPUT_SIZE ? 0 : (isize) + ((isize)/255) + 16)

/*! LZ4_compressBound() :
    Provides the maximum size that LZ4 compression may output in a "worst case" scenario (input data not compressible)
    This function is primarily useful for memory allocation purposes (destination buffer size).
    Macro LZ4_COMPRESSBOUND() is also provided for compilation-time evaluation (stack memory allocation for example).
    Note that LZ4_compress_default() compresses faster when dstCapacity is >= LZ4_compressBound(srcSize)
        inputSize  : max supported value is LZ4_MAX_INPUT_SIZE
        return : maximum output size in a "worst case" scenario
              or 0, if input size is incorrect (too large or negative)
*/
LZ4LIB_API int LZ4_compressBound(int inputSize);

/*! LZ4_compress_fast() :
    Same as LZ4_compress_default(), but allows selection of "acceleration" factor.
    The larger the acceleration value, the faster the algorithm, but also the lesser the compression.
    It's a trade-off. It can be fine tuned, with each successive value providing roughly +~3% to speed.
    An acceleration value of "1" is the same as regular LZ4_compress_default()
    Values <= 0 will be replaced by ACCELERATION_DEFAULT (currently == 1, see lz4.c).
*/
LZ4LIB_API int LZ4_compress_fast (const char* src, char* dst, int srcSize, int dstCapacity, int acceleration);


/*! LZ4_compress_fast_extState() :
 *  Same as LZ4_compress_fast(), using an externally allocated memory space for its state.
 *  Use LZ4_sizeofState() to know how much memory must be allocated,
 *  and allocate it on 8-bytes boundaries (using `malloc()` typically).
 *  Then, provide this buffer as `void* state` to compression function.
 */
LZ4LIB_API int LZ4_sizeofState(void);
LZ4LIB_API int LZ4_compress_fast_extState (void* state, const char* src, char* dst, int srcSize, int dstCapacity, int acceleration);


/*! LZ4_compress_destSize() :
 *  Reverse the logic : compresses as much data as possible from 'src' buffer
 *  into already allocated buffer 'dst', of size >= 'targetDestSize'.
 *  This function either compresses the entire 'src' content into 'dst' if it's large enough,
 *  or fill 'dst' buffer completely with as much data as possible from 'src'.
 *  note: acceleration parameter is fixed to "default".
 *
 * *srcSizePtr : will be modified to indicate how many bytes where read from 'src' to fill 'dst'.
 *               New value is necessarily <= input value.
 * @return : Nb bytes written into 'dst' (necessarily <= targetDestSize)
 *           or 0 if compression fails.
*/
LZ4LIB_API int LZ4_compress_destSize (const char* src, char* dst, int* srcSizePtr, int targetDstSize);


/*! LZ4_decompress_safe_partial() :
 *  Decompress an LZ4 compressed block, of size 'srcSize' at position 'src',
 *  into destination buffer 'dst' of size 'dstCapacity'.
 *  Up to 'targetOutputSize' bytes will be decoded.
 *  The function stops decoding on reaching this objective,
 *  which can boost performance when only the beginning of a block is required.
 *
 * @return : the number of bytes decoded in `dst` (necessarily <= dstCapacity)
 *           If source stream is detected malformed, function returns a negative result.
 *
 *  Note : @return can be < targetOutputSize, if compressed block contains less data.
 *
 *  Note 2 : this function features 2 parameters, targetOutputSize and dstCapacity,
 *           and expects targetOutputSize <= dstCapacity.
 *           It effectively stops decoding on reaching targetOutputSize,
 *           so dstCapacity is kind of redundant.
 *           This is because in a previous version of this function,
 *           decoding operation would not "break" a sequence in the middle.
 *           As a consequence, there was no guarantee that decoding would stop at exactly targetOutputSize,
 *           it could write more bytes, though only up to dstCapacity.
 *           Some "margin" used to be required for this operation to work properly.
 *           This is no longer necessary.
 *           The function nonetheless keeps its signature, in an effort to not break API.
 */
LZ4LIB_API int LZ4_decompress_safe_partial (const char* src, char* dst, int srcSize, int targetOutputSize, int dstCapacity);


/*-*********************************************
*  Streaming Compression Functions
***********************************************/
typedef union LZ4_stream_u LZ4_stream_t;  /* incomplete type (defined later) */

LZ4LIB_API LZ4_stream_t* LZ4_createStream(void);
LZ4LIB_API int           LZ4_freeStream (LZ4_stream_t* streamPtr);

/*! LZ4_resetStream_fast() : v1.9.0+
 *  Use this to prepare an LZ4_stream_t for a new chain of dependent blocks
 *  (e.g., LZ4_compress_fast_continue()).
 *
 *  An LZ4_stream_t must be initialized once before usage.
 *  This is automatically done when created by LZ4_createStream().
 *  However, should the LZ4_stream_t be simply declared on stack (for example),
 *  it's necessary to initialize it first, using LZ4_initStream().
 *
 *  After init, start any new stream with LZ4_resetStream_fast().
 *  A same LZ4_stream_t can be re-used multiple times consecutively
 *  and compress multiple streams,
 *  provided that it starts each new stream with LZ4_resetStream_fast().
 *
 *  LZ4_resetStream_fast() is much faster than LZ4_initStream(),
 *  but is not compatible with memory regions containing garbage data.
 *
 *  Note: it's only useful to call LZ4_resetStream_fast()
 *        in the context of streaming compression.
 *        The *extState* functions perform their own resets.
 *        Invoking LZ4_resetStream_fast() before is redundant, and even counterproductive.
 */
LZ4LIB_API void LZ4_resetStream_fast (LZ4_stream_t* streamPtr);

/*! LZ4_loadDict() :
 *  Use this function to reference a static dictionary into LZ4_stream_t.
 *  The dictionary must remain available during compression.
 *  LZ4_loadDict() triggers a reset, so any previous data will be forgotten.
 *  The same dictionary will have to be loaded on decompression side for successful decoding.
 *  Dictionary are useful for better compression of small data (KB range).
 *  While LZ4 accept any input as dictionary,
 *  results are generally better when using Zstandard's Dictionary Builder.
 *  Loading a size of 0 is allowed, and is the same as reset.
 * @return : loaded dictionary size, in bytes (necessarily <= 64 KB)
 */
LZ4LIB_API int LZ4_loadDict (LZ4_stream_t* streamPtr, const char* dictionary, int dictSize);

/*! LZ4_compress_fast_continue() :
 *  Compress 'src' content using data from previously compressed blocks, for better compression ratio.
 * 'dst' buffer must be already allocated.
 *  If dstCapacity >= LZ4_compressBound(srcSize), compression is guaranteed to succeed, and runs faster.
 *
 * @return : size of compressed block
 *           or 0 if there is an error (typically, cannot fit into 'dst').
 *
 *  Note 1 : Each invocation to LZ4_compress_fast_continue() generates a new block.
 *           Each block has precise boundaries.
 *           Each block must be decompressed separately, calling LZ4_decompress_*() with relevant metadata.
 *           It's not possible to append blocks together and expect a single invocation of LZ4_decompress_*() to decompress them together.
 *
 *  Note 2 : The previous 64KB of source data is __assumed__ to remain present, unmodified, at same address in memory !
 *
 *  Note 3 : When input is structured as a double-buffer, each buffer can have any size, including < 64 KB.
 *           Make sure that buffers are separated, by at least one byte.
 *           This construction ensures that each block only depends on previous block.
 *
 *  Note 4 : If input buffer is a ring-buffer, it can have any size, including < 64 KB.
 *
 *  Note 5 : After an error, the stream status is undefined (invalid), it can only be reset or freed.
 */
LZ4LIB_API int LZ4_compress_fast_continue (LZ4_stream_t* streamPtr, const char* src, char* dst, int srcSize, int dstCapacity, int acceleration);

/*! LZ4_saveDict() :
 *  If last 64KB data cannot be guaranteed to remain available at its current memory location,
 *  save it into a safer place (char* safeBuffer).
 *  This is schematically equivalent to a memcpy() followed by LZ4_loadDict(),
 *  but is much faster, because LZ4_saveDict() doesn't need to rebuild tables.
 * @return : saved dictionary size in bytes (necessarily <= maxDictSize), or 0 if error.
 */
LZ4LIB_API int LZ4_saveDict (LZ4_stream_t* streamPtr, char* safeBuffer, int maxDictSize);


/*-**********************************************
*  Streaming Decompression Functions
*  Bufferless synchronous API
************************************************/
typedef union LZ4_streamDecode_u LZ4_streamDecode_t;   /* tracking context */

/*! LZ4_createStreamDecode() and LZ4_freeStreamDecode() :
 *  creation / destruction of streaming decompression tracking context.
 *  A tracking context can be re-used multiple times.
 */
LZ4LIB_API LZ4_streamDecode_t* LZ4_createStreamDecode(void);
LZ4LIB_API int                 LZ4_freeStreamDecode (LZ4_streamDecode_t* LZ4_stream);

/*! LZ4_setStreamDecode() :
 *  An LZ4_streamDecode_t context can be allocated once and re-used multiple times.
 *  Use this function to start decompression of a new stream of blocks.
 *  A dictionary can optionally be set. Use NULL or size 0 for a reset order.
 *  Dictionary is presumed stable : it must remain accessible and unmodified during next decompression.
 * @return : 1 if OK, 0 if error
 */
LZ4LIB_API int LZ4_setStreamDecode (LZ4_streamDecode_t* LZ4_streamDecode, const char* dictionary, int dictSize);

/*! LZ4_decoderRingBufferSize() : v1.8.2+
 *  Note : in a ring buffer scenario (optional),
 *  blocks are presumed decompressed next to each other
 *  up to the moment there is not enough remaining space for next block (remainingSize < maxBlockSize),
 *  at which stage it resumes from beginning of ring buffer.
 *  When setting such a ring buffer for streaming decompression,
 *  provides the minimum size of this ring buffer
 *  to be compatible with any source respecting maxBlockSize condition.
 * @return : minimum ring buffer size,
 *           or 0 if there is an error (invalid maxBlockSize).
 */
LZ4LIB_API int LZ4_decoderRingBufferSize(int maxBlockSize);
#define LZ4_DECODER_RING_BUFFER_SIZE(maxBlockSize) (65536 + 14 + (maxBlockSize))  /* for static allocation; maxBlockSize presumed valid */

/*! LZ4_decompress_*_continue() :
 *  These decoding functions allow decompression of consecutive blocks in "streaming" mode.
 *  A block is an unsplittable entity, it must be presented entirely to a decompression function.
 *  Decompression functions only accepts one block at a time.
 *  The last 64KB of previously decoded data *must* remain available and unmodified at the memory position where they were decoded.
 *  If less than 64KB of data has been decoded, all the data must be present.
 *
 *  Special : if decompression side sets a ring buffer, it must respect one of the following conditions :
 *  - Decompression buffer size is _at least_ LZ4_decoderRingBufferSize(maxBlockSize).
 *    maxBlockSize is the maximum size of any single block. It can have any value > 16 bytes.
 *    In which case, encoding and decoding buffers do not need to be synchronized.
 *    Actually, data can be produced by any source compliant with LZ4 format specification, and respecting maxBlockSize.
 *  - Synchronized mode :
 *    Decompression buffer size is _exactly_ the same as compression buffer size,
 *    and follows exactly same update rule (block boundaries at same positions),
 *    and decoding function is provided with exact decompressed size of each block (exception for last block of the stream),
 *    _then_ decoding & encoding ring buffer can have any size, including small ones ( < 64 KB).
 *  - Decompression buffer is larger than encoding buffer, by a minimum of maxBlockSize more bytes.
 *    In which case, encoding and decoding buffers do not need to be synchronized,
 *    and encoding ring buffer can have any size, including small ones ( < 64 KB).
 *
 *  Whenever these conditions are not possible,
 *  save the last 64KB of decoded data into a safe buffer where it can't be modified during decompression,
 *  then indicate where this data is saved using LZ4_setStreamDecode(), before decompressing next block.
*/
LZ4LIB_API int LZ4_decompress_safe_continue (LZ4_streamDecode_t* LZ4_streamDecode, const char* src, char* dst, int srcSize, int dstCapacity);


/*! LZ4_decompress_*_usingDict() :
 *  These decoding functions work the same as
 *  a combination of LZ4_setStreamDecode() followed by LZ4_decompress_*_continue()
 *  They are stand-alone, and don't need an LZ4_streamDecode_t structure.
 *  Dictionary is presumed stable : it must remain accessible and unmodified during decompression.
 *  Performance tip : Decompression speed can be substantially increased
 *                    when dst == dictStart + dictSize.
 */
LZ4LIB_API int LZ4_decompress_safe_usingDict (const char* src, char* dst, int srcSize, int dstCapcity, const char* dictStart, int dictSize);

LZ4_END_NAMESPACE // EPIC MOD : Wrap library in an optional namespace

#endif /* LZ4_H_2983827168210 */


/*^*************************************
 * !!!!!!   STATIC LINKING ONLY   !!!!!!
 ***************************************/

/*-****************************************************************************
 * Experimental section
 *
 * Symbols declared in this section must be considered unstable. Their
 * signatures or semantics may change, or they may be removed altogether in the
 * future. They are therefore only safe to depend on when the caller is
 * statically linked against the library.
 *
 * To protect against unsafe usage, not only are the declarations guarded,
 * the definitions are hidden by default
 * when building LZ4 as a shared/dynamic library.
 *
 * In order to access these declarations,
 * define LZ4_STATIC_LINKING_ONLY in your application
 * before including LZ4's headers.
 *
 * In order to make their implementations accessible dynamically, you must
 * define LZ4_PUBLISH_STATIC_FUNCTIONS when building the LZ4 library.
 ******************************************************************************/

#ifdef LZ4_STATIC_LINKING_ONLY

#ifndef LZ4_STATIC_3504398509
#define LZ4_STATIC_3504398509

#ifdef LZ4_PUBLISH_STATIC_FUNCTIONS
#define LZ4LIB_STATIC_API LZ4LIB_API
#else
#define LZ4LIB_STATIC_API
#endif

LZ4_BEGIN_NAMESPACE // EPIC MOD : Wrap library in an optional namespace


/*! LZ4_compress_fast_extState_fastReset() :
 *  A variant of LZ4_compress_fast_extState().
 *
 *  Using this variant avoids an expensive initialization step.
 *  It is only safe to call if the state buffer is known to be correctly initialized already
 *  (see above comment on LZ4_resetStream_fast() for a definition of "correctly initialized").
 *  From a high level, the difference is that
 *  this function initializes the provided state with a call to something like LZ4_resetStream_fast()
 *  while LZ4_compress_fast_extState() starts with a call to LZ4_resetStream().
 */
LZ4LIB_STATIC_API int LZ4_compress_fast_extState_fastReset (void* state, const char* src, char* dst, int srcSize, int dstCapacity, int acceleration);

/*! LZ4_attach_dictionary() :
 *  This is an experimental API that allows
 *  efficient use of a static dictionary many times.
 *
 *  Rather than re-loading the dictionary buffer into a working context before
 *  each compression, or copying a pre-loaded dictionary's LZ4_stream_t into a
 *  working LZ4_stream_t, this function introduces a no-copy setup mechanism,
 *  in which the working stream references the dictionary stream in-place.
 *
 *  Several assumptions are made about the state of the dictionary stream.
 *  Currently, only streams which have been prepared by LZ4_loadDict() should
 *  be expected to work.
 *
 *  Alternatively, the provided dictionaryStream may be NULL,
 *  in which case any existing dictionary stream is unset.
 *
 *  If a dictionary is provided, it replaces any pre-existing stream history.
 *  The dictionary contents are the only history that can be referenced and
 *  logically immediately precede the data compressed in the first subsequent
 *  compression call.
 *
 *  The dictionary will only remain attached to the working stream through the
 *  first compression call, at the end of which it is cleared. The dictionary
 *  stream (and source buffer) must remain in-place / accessible / unchanged
 *  through the completion of the first compression call on the stream.
 */
LZ4LIB_STATIC_API void LZ4_attach_dictionary(LZ4_stream_t* workingStream, const LZ4_stream_t* dictionaryStream);


/*! In-place compression and decompression
 *
 * It's possible to have input and output sharing the same buffer,
 * for highly contrained memory environments.
 * In both cases, it requires input to lay at the end of the buffer,
 * and decompression to start at beginning of the buffer.
 * Buffer size must feature some margin, hence be larger than final size.
 *
 * |<------------------------buffer--------------------------------->|
 *                             |<-----------compressed data--------->|
 * |<-----------decompressed size------------------>|
 *                                                  |<----margin---->|
 *
 * This technique is more useful for decompression,
 * since decompressed size is typically larger,
 * and margin is short.
 *
 * In-place decompression will work inside any buffer
 * which size is >= LZ4_DECOMPRESS_INPLACE_BUFFER_SIZE(decompressedSize).
 * This presumes that decompressedSize > compressedSize.
 * Otherwise, it means compression actually expanded data,
 * and it would be more efficient to store such data with a flag indicating it's not compressed.
 * This can happen when data is not compressible (already compressed, or encrypted).
 *
 * For in-place compression, margin is larger, as it must be able to cope with both
 * history preservation, requiring input data to remain unmodified up to LZ4_DISTANCE_MAX,
 * and data expansion, which can happen when input is not compressible.
 * As a consequence, buffer size requirements are much higher,
 * and memory savings offered by in-place compression are more limited.
 *
 * There are ways to limit this cost for compression :
 * - Reduce history size, by modifying LZ4_DISTANCE_MAX.
 *   Note that it is a compile-time constant, so all compressions will apply this limit.
 *   Lower values will reduce compression ratio, except when input_size < LZ4_DISTANCE_MAX,
 *   so it's a reasonable trick when inputs are known to be small.
 * - Require the compressor to deliver a "maximum compressed size".
 *   This is the `dstCapacity` parameter in `LZ4_compress*()`.
 *   When this size is < LZ4_COMPRESSBOUND(inputSize), then compression can fail,
 *   in which case, the return code will be 0 (zero).
 *   The caller must be ready for these cases to happen,
 *   and typically design a backup scheme to send data uncompressed.
 * The combination of both techniques can significantly reduce
 * the amount of margin required for in-place compression.
 *
 * In-place compression can work in any buffer
 * which size is >= (maxCompressedSize)
 * with maxCompressedSize == LZ4_COMPRESSBOUND(srcSize) for guaranteed compression success.
 * LZ4_COMPRESS_INPLACE_BUFFER_SIZE() depends on both maxCompressedSize and LZ4_DISTANCE_MAX,
 * so it's possible to reduce memory requirements by playing with them.
 */

#define LZ4_DECOMPRESS_INPLACE_MARGIN(compressedSize)          (((compressedSize) >> 8) + 32)
#define LZ4_DECOMPRESS_INPLACE_BUFFER_SIZE(decompressedSize)   ((decompressedSize) + LZ4_DECOMPRESS_INPLACE_MARGIN(decompressedSize))  /**< note: presumes that compressedSize < decompressedSize. note2: margin is overestimated a bit, since it could use compressedSize instead */

#ifndef LZ4_DISTANCE_MAX   /* history window size; can be user-defined at compile time */
#  define LZ4_DISTANCE_MAX 65535   /* set to maximum value by default */
#endif

#define LZ4_COMPRESS_INPLACE_MARGIN                           (LZ4_DISTANCE_MAX + 32)   /* LZ4_DISTANCE_MAX can be safely replaced by srcSize when it's smaller */
#define LZ4_COMPRESS_INPLACE_BUFFER_SIZE(maxCompressedSize)   ((maxCompressedSize) + LZ4_COMPRESS_INPLACE_MARGIN)  /**< maxCompressedSize is generally LZ4_COMPRESSBOUND(inputSize), but can be set to any lower value, with the risk that compression can fail (return code 0(zero)) */

LZ4_END_NAMESPACE // EPIC MOD : Wrap library in an optional namespace

#endif   /* LZ4_STATIC_3504398509 */
#endif   /* LZ4_STATIC_LINKING_ONLY */



#ifndef LZ4_H_98237428734687
#define LZ4_H_98237428734687

/*-************************************************************
 *  PRIVATE DEFINITIONS
 **************************************************************
 * Do not use these definitions directly.
 * They are only exposed to allow static allocation of `LZ4_stream_t` and `LZ4_streamDecode_t`.
 * Accessing members will expose code to API and/or ABI break in future versions of the library.
 **************************************************************/
#define LZ4_HASHLOG   (LZ4_MEMORY_USAGE-2)
#define LZ4_HASHTABLESIZE (1 << LZ4_MEMORY_USAGE)
#define LZ4_HASH_SIZE_U32 (1 << LZ4_HASHLOG)       /* required as macro for static allocation */

#if defined(__cplusplus) || (defined (__STDC_VERSION__) && (__STDC_VERSION__ >= 199901L) /* C99 */)
#include <stdint.h>

LZ4_BEGIN_NAMESPACE // EPIC MOD : Wrap library in an optional namespace

typedef struct LZ4_stream_t_internal LZ4_stream_t_internal;
struct LZ4_stream_t_internal {
    uint32_t hashTable[LZ4_HASH_SIZE_U32];
    uint32_t currentOffset;
    uint16_t dirty;
    uint16_t tableType;
    const uint8_t* dictionary;
    const LZ4_stream_t_internal* dictCtx;
    uint32_t dictSize;
};

typedef struct {
    const uint8_t* externalDict;
    size_t extDictSize;
    const uint8_t* prefixEnd;
    size_t prefixSize;
} LZ4_streamDecode_t_internal;

#else

typedef struct LZ4_stream_t_internal LZ4_stream_t_internal;
struct LZ4_stream_t_internal {
    unsigned int hashTable[LZ4_HASH_SIZE_U32];
    unsigned int currentOffset;
    unsigned short dirty;
    unsigned short tableType;
    const unsigned char* dictionary;
    const LZ4_stream_t_internal* dictCtx;
    unsigned int dictSize;
};

typedef struct {
    const unsigned char* externalDict;
    const unsigned char* prefixEnd;
    size_t extDictSize;
    size_t prefixSize;
} LZ4_streamDecode_t_internal;

#endif

/*! LZ4_stream_t :
 *  information structure to track an LZ4 stream.
 *  LZ4_stream_t can also be created using LZ4_createStream(), which is recommended.
 *  The structure definition can be convenient for static allocation
 *  (on stack, or as part of larger structure).
 *  Init this structure with LZ4_initStream() before first use.
 *  note : only use this definition in association with static linking !
 *    this definition is not API/ABI safe, and may change in a future version.
 */
#define LZ4_STREAMSIZE_U64 ((1 << (LZ4_MEMORY_USAGE-3)) + 4 + ((sizeof(void*)==16) ? 4 : 0) /*AS-400*/ )
#define LZ4_STREAMSIZE     (LZ4_STREAMSIZE_U64 * sizeof(unsigned long long))
union LZ4_stream_u {
    unsigned long long table[LZ4_STREAMSIZE_U64];
    LZ4_stream_t_internal internal_donotuse;
} ;  /* previously typedef'd to LZ4_stream_t */

/*! LZ4_initStream() : v1.9.0+
 *  An LZ4_stream_t structure must be initialized at least once.
 *  This is automatically done when invoking LZ4_createStream(),
 *  but it's not when the structure is simply declared on stack (for example).
 *
 *  Use LZ4_initStream() to properly initialize a newly declared LZ4_stream_t.
 *  It can also initialize any arbitrary buffer of sufficient size,
 *  and will @return a pointer of proper type upon initialization.
 *
 *  Note : initialization fails if size and alignment conditions are not respected.
 *         In which case, the function will @return NULL.
 *  Note2: An LZ4_stream_t structure guarantees correct alignment and size.
 *  Note3: Before v1.9.0, use LZ4_resetStream() instead
 */
LZ4LIB_API LZ4_stream_t* LZ4_initStream (void* buffer, size_t size);


/*! LZ4_streamDecode_t :
 *  information structure to track an LZ4 stream during decompression.
 *  init this structure  using LZ4_setStreamDecode() before first use.
 *  note : only use in association with static linking !
 *         this definition is not API/ABI safe,
 *         and may change in a future version !
 */
#define LZ4_STREAMDECODESIZE_U64 (4 + ((sizeof(void*)==16) ? 2 : 0) /*AS-400*/ )
#define LZ4_STREAMDECODESIZE     (LZ4_STREAMDECODESIZE_U64 * sizeof(unsigned long long))
union LZ4_streamDecode_u {
    unsigned long long table[LZ4_STREAMDECODESIZE_U64];
    LZ4_streamDecode_t_internal internal_donotuse;
} ;   /* previously typedef'd to LZ4_streamDecode_t */



/*-************************************
*  Obsolete Functions
**************************************/

/*! Deprecation warnings
 *
 *  Deprecated functions make the compiler generate a warning when invoked.
 *  This is meant to invite users to update their source code.
 *  Should deprecation warnings be a problem, it is generally possible to disable them,
 *  typically with -Wno-deprecated-declarations for gcc
 *  or _CRT_SECURE_NO_WARNINGS in Visual.
 *
 *  Another method is to define LZ4_DISABLE_DEPRECATE_WARNINGS
 *  before including the header file.
 */
#ifdef LZ4_DISABLE_DEPRECATE_WARNINGS
#  define LZ4_DEPRECATED(message)   /* disable deprecation warnings */
#else
#  define LZ4_GCC_VERSION (__GNUC__ * 100 + __GNUC_MINOR__)
#  if defined (__cplusplus) && (__cplusplus >= 201402) /* C++14 or greater */
#    define LZ4_DEPRECATED(message) [[deprecated(message)]]
#  elif (LZ4_GCC_VERSION >= 405) || defined(__clang__)
#    define LZ4_DEPRECATED(message) __attribute__((deprecated(message)))
#  elif (LZ4_GCC_VERSION >= 301)
#    define LZ4_DEPRECATED(message) __attribute__((deprecated))
#  elif defined(_MSC_VER)
#    define LZ4_DEPRECATED(message) __declspec(deprecated(message))
#  else
#    pragma message("WARNING: You need to implement LZ4_DEPRECATED for this compiler")
#    define LZ4_DEPRECATED(message)
#  endif
#endif /* LZ4_DISABLE_DEPRECATE_WARNINGS */

/* Obsolete compression functions */
LZ4_DEPRECATED("use LZ4_compress_default() instead")       LZ4LIB_API int LZ4_compress               (const char* src, char* dest, int srcSize);
LZ4_DEPRECATED("use LZ4_compress_default() instead")       LZ4LIB_API int LZ4_compress_limitedOutput (const char* src, char* dest, int srcSize, int maxOutputSize);
LZ4_DEPRECATED("use LZ4_compress_fast_extState() instead") LZ4LIB_API int LZ4_compress_withState               (void* state, const char* source, char* dest, int inputSize);
LZ4_DEPRECATED("use LZ4_compress_fast_extState() instead") LZ4LIB_API int LZ4_compress_limitedOutput_withState (void* state, const char* source, char* dest, int inputSize, int maxOutputSize);
LZ4_DEPRECATED("use LZ4_compress_fast_continue() instead") LZ4LIB_API int LZ4_compress_continue                (LZ4_stream_t* LZ4_streamPtr, const char* source, char* dest, int inputSize);
LZ4_DEPRECATED("use LZ4_compress_fast_continue() instead") LZ4LIB_API int LZ4_compress_limitedOutput_continue  (LZ4_stream_t* LZ4_streamPtr, const char* source, char* dest, int inputSize, int maxOutputSize);

/* Obsolete decompression functions */
LZ4_DEPRECATED("use LZ4_decompress_fast() instead") LZ4LIB_API int LZ4_uncompress (const char* source, char* dest, int outputSize);
LZ4_DEPRECATED("use LZ4_decompress_safe() instead") LZ4LIB_API int LZ4_uncompress_unknownOutputSize (const char* source, char* dest, int isize, int maxOutputSize);

/* Obsolete streaming functions; degraded functionality; do not use!
 *
 * In order to perform streaming compression, these functions depended on data
 * that is no longer tracked in the state. They have been preserved as well as
 * possible: using them will still produce a correct output. However, they don't
 * actually retain any history between compression calls. The compression ratio
 * achieved will therefore be no better than compressing each chunk
 * independently.
 */
LZ4_DEPRECATED("Use LZ4_createStream() instead") LZ4LIB_API void* LZ4_create (char* inputBuffer);
LZ4_DEPRECATED("Use LZ4_createStream() instead") LZ4LIB_API int   LZ4_sizeofStreamState(void);
LZ4_DEPRECATED("Use LZ4_resetStream() instead")  LZ4LIB_API int   LZ4_resetStreamState(void* state, char* inputBuffer);
LZ4_DEPRECATED("Use LZ4_saveDict() instead")     LZ4LIB_API char* LZ4_slideInputBuffer (void* state);

/* Obsolete streaming decoding functions */
LZ4_DEPRECATED("use LZ4_decompress_safe_usingDict() instead") LZ4LIB_API int LZ4_decompress_safe_withPrefix64k (const char* src, char* dst, int compressedSize, int maxDstSize);
LZ4_DEPRECATED("use LZ4_decompress_fast_usingDict() instead") LZ4LIB_API int LZ4_decompress_fast_withPrefix64k (const char* src, char* dst, int originalSize);

/*! LZ4_decompress_fast() : **unsafe!**
 *  These functions used to be faster than LZ4_decompress_safe(),
 *  but it has changed, and they are now slower than LZ4_decompress_safe().
 *  This is because LZ4_decompress_fast() doesn't know the input size,
 *  and therefore must progress more cautiously in the input buffer to not read beyond the end of block.
 *  On top of that `LZ4_decompress_fast()` is not protected vs malformed or malicious inputs, making it a security liability.
 *  As a consequence, LZ4_decompress_fast() is strongly discouraged, and deprecated.
 *
 *  The last remaining LZ4_decompress_fast() specificity is that
 *  it can decompress a block without knowing its compressed size.
 *  Such functionality could be achieved in a more secure manner,
 *  by also providing the maximum size of input buffer,
 *  but it would require new prototypes, and adaptation of the implementation to this new use case.
 *
 *  Parameters:
 *  originalSize : is the uncompressed size to regenerate.
 *                 `dst` must be already allocated, its size must be >= 'originalSize' bytes.
 * @return : number of bytes read from source buffer (== compressed size).
 *           The function expects to finish at block's end exactly.
 *           If the source stream is detected malformed, the function stops decoding and returns a negative result.
 *  note : LZ4_decompress_fast*() requires originalSize. Thanks to this information, it never writes past the output buffer.
 *         However, since it doesn't know its 'src' size, it may read an unknown amount of input, past input buffer bounds.
 *         Also, since match offsets are not validated, match reads from 'src' may underflow too.
 *         These issues never happen if input (compressed) data is correct.
 *         But they may happen if input data is invalid (error or intentional tampering).
 *         As a consequence, use these functions in trusted environments with trusted data **only**.
 */

LZ4_DEPRECATED("This function is deprecated and unsafe. Consider using LZ4_decompress_safe() instead")
LZ4LIB_API int LZ4_decompress_fast (const char* src, char* dst, int originalSize);
LZ4_DEPRECATED("This function is deprecated and unsafe. Consider using LZ4_decompress_safe_continue() instead")
LZ4LIB_API int LZ4_decompress_fast_continue (LZ4_streamDecode_t* LZ4_streamDecode, const char* src, char* dst, int originalSize);
LZ4_DEPRECATED("This function is deprecated and unsafe. Consider using LZ4_decompress_safe_usingDict() instead")
LZ4LIB_API int LZ4_decompress_fast_usingDict (const char* src, char* dst, int originalSize, const char* dictStart, int dictSize);

/*! LZ4_resetStream() :
 *  An LZ4_stream_t structure must be initialized at least once.
 *  This is done with LZ4_initStream(), or LZ4_resetStream().
 *  Consider switching to LZ4_initStream(),
 *  invoking LZ4_resetStream() will trigger deprecation warnings in the future.
 */
LZ4LIB_API void LZ4_resetStream (LZ4_stream_t* streamPtr);

LZ4_END_NAMESPACE // EPIC MOD : Wrap library in an optional namespace


#endif /* LZ4_H_98237428734687 */


#if defined (__cplusplus)
}
#endif
