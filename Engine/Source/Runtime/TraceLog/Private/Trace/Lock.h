// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Trace/Config.h"
#include "Trace/Detail/Atomic.h"
#include "Platform.h"

#if TRACE_PRIVATE_MINIMAL_ENABLED

namespace UE {
namespace Trace {
namespace Private {

struct FLock 
{
	FLock(volatile uint8* InLock)
		: Lock(InLock)
	{
		CyclesPerSecond = TimeGetFrequency();
		StartSeconds = GetTime();

		while (!AtomicCompareExchangeAcquire(Lock, uint8(1u), uint8(0u)))
		{
			ThreadSleep(0);

			if (TimedOut())
			{
				break;
			}
		}
	}

	~FLock()
	{
		AtomicExchangeRelease(Lock, uint8(0u));
	}

	double GetTime()
	{
		return static_cast<double>(TimeGetTimestamp()) / static_cast<double>(CyclesPerSecond);
	}

	bool TimedOut()
	{
		const double WaitTime = GetTime() - StartSeconds;
		return WaitTime > MaxWaitSeconds;
	}

	volatile uint8* Lock;
	uint64 CyclesPerSecond;
	double StartSeconds;
	inline const static double MaxWaitSeconds = 1.0;
};

}}}

#endif
