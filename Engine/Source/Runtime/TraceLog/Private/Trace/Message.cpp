// Copyright Epic Games, Inc. All Rights Reserved.

#include "Message.h"

#if TRACE_PRIVATE_MINIMAL_ENABLED

#include "Trace/Trace.h"

namespace UE {
namespace Trace {
namespace Private {

////////////////////////////////////////////////////////////////////////////////
static OnMessageFunc* GMessageFunc = nullptr;

////////////////////////////////////////////////////////////////////////////////
void Message_SetCallback(OnMessageFunc* Callback)
{
	GMessageFunc = Callback;
}

////////////////////////////////////////////////////////////////////////////////
void Message_Send(EMessageType Type, const char* TypeStr, const char* Description /* = nullptr */)
{
	if (!GMessageFunc)
	{
		return;
	}

	FMessageEvent Message { Type, TypeStr, Description };
	GMessageFunc(Message);
}

} } } // namespace UE::Trace::Private

#endif // TRACE_PRIVATE_MINIMAL_ENABLED