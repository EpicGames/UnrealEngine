// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Trace/Config.h"
#include "Trace/Trace.h"
#include <stdio.h>

#if TRACE_PRIVATE_MINIMAL_ENABLED

namespace UE {
namespace Trace {
namespace Private {

bool GetErrorMessage(char* OutBuffer, uint32 BufferSize, int32 ErrorCode);

/**
 * Utility macro to format messages
 */

#define UE_TRACE_MESSAGE_FMT_MAX_SIZE 512
#define UE_TRACE_MESSAGE_ERR_MAX_SIZE 256

#define UE_TRACE_MESSAGE(Type, Msg) \
	Message_Send(EMessageType::Type, #Type, Msg);

#define UE_TRACE_MESSAGE_F(Type, Fmt, ...) \
	{\
		char Buff[UE_TRACE_MESSAGE_FMT_MAX_SIZE];\
		snprintf(Buff, UE_TRACE_MESSAGE_FMT_MAX_SIZE, Fmt, __VA_ARGS__);\
		Message_Send(EMessageType::Type, #Type, Buff);\
	}

#define UE_TRACE_ERRORMESSAGE(Type, ErrorCode)\
	{\
		char ErrorMessageBuffer[UE_TRACE_MESSAGE_ERR_MAX_SIZE] = {'\0'};\
		GetErrorMessage(ErrorMessageBuffer, UE_TRACE_MESSAGE_ERR_MAX_SIZE, ErrorCode);\
		UE_TRACE_MESSAGE_F(Type, "(error code %d): '%s'", ErrorCode, ErrorMessageBuffer)\
	}

#define UE_TRACE_ERRORMESSAGE_F(Type, ErrorCode, ContextFmt, ...)\
	{\
		char ErrorMessageBuffer[UE_TRACE_MESSAGE_ERR_MAX_SIZE];\
		if (GetErrorMessage(ErrorMessageBuffer, UE_TRACE_MESSAGE_ERR_MAX_SIZE, ErrorCode))\
		{\
			char FinalBuff[UE_TRACE_MESSAGE_FMT_MAX_SIZE];\
			snprintf(FinalBuff, UE_TRACE_MESSAGE_FMT_MAX_SIZE, ContextFmt ": '%s'", __VA_ARGS__, ErrorMessageBuffer);\
			Message_Send(EMessageType::Type, #Type, FinalBuff);\
		}\
		else\
		{\
			UE_TRACE_MESSAGE_F(Type, "(error code %d)", ErrorCode)\
		}\
	}

/**
 * Initialize messages. Saves the provided callback if specified.
 */
void Message_SetCallback(UE::Trace::OnMessageFunc* Callback);

/**
 * Sends a message to the registered callback. Description string is not
 * expected to outlive the function call.
 */
void Message_Send(EMessageType Type, const char* TypeStr, const char* Description = nullptr);


} } } // namespace UE::Trace::Private

#endif // TRACE_PRIVATE_MINIMAL_ENABLED
