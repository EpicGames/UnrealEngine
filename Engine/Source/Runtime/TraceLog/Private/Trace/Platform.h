// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Trace/Config.h"
#include "Trace/Trace.h"

#if TRACE_PRIVATE_MINIMAL_ENABLED

namespace UE {
namespace Trace {
namespace Private {

////////////////////////////////////////////////////////////////////////////////
UPTRINT	ThreadCreate(const ANSICHAR* Name, void (*Entry)());
void	ThreadSleep(uint32 Milliseconds);
void	ThreadJoin(UPTRINT Handle);
void	ThreadDestroy(UPTRINT Handle);

////////////////////////////////////////////////////////////////////////////////
uint64				TimeGetFrequency();
/**
 * Gets the absolute timestamp of the system.
 * @return Absolute timestamp
 */
TRACELOG_API uint64	TimeGetTimestamp();
/**
 * Gets the timestamp (relative to base). If called
 * before Trace initialization value will be zero.
 * @return Timestamp relative base time stamp
 */
TRACELOG_API uint64 TimeGetRelativeTimestamp();

////////////////////////////////////////////////////////////////////////////////
UPTRINT	TcpSocketConnect(const ANSICHAR* Host, uint16 Port);
UPTRINT	TcpSocketListen(uint16 Port);
int32	TcpSocketAccept(UPTRINT Socket, UPTRINT& Out, bool bAsNonBlocking = false);
bool	TcpSocketHasData(UPTRINT Socket);
int		TcpSocketNative(UPTRINT Socket);
bool	TcpSocketPeerName(UPTRINT Socket, char* Name, uint32 NameSize);
bool	TcpSocketSetNonBlocking(UPTRINT Socket, bool bNonBlocking);

////////////////////////////////////////////////////////////////////////////////
int32	IoRead(UPTRINT Handle, void* Data, uint32 Size);
bool	IoWrite(UPTRINT Handle, const void* Data, uint32 Size);
void	IoClose(UPTRINT Handle);

////////////////////////////////////////////////////////////////////////////////
UPTRINT	FileOpen(const ANSICHAR* Path);
	
////////////////////////////////////////////////////////////////////////////////
int32	GetLastErrorCode();
bool	GetErrorMessage(char* OutBuffer, uint32 BufferSize, int32 ErrorCode);	

////////////////////////////////////////////////////////////////////////////////
#if !defined(TRACE_PRIVATE_HAS_THROTTLE)
#	define	TRACE_PRIVATE_HAS_THROTTLE	0
#endif

#if TRACE_PRIVATE_HAS_THROTTLE
#	define	THROTTLE_IMPL(x)	;
#	define	THROTTLE_INL
#else
#	define	THROTTLE_IMPL(x)	{x}
#	define	THROTTLE_INL		inline
#endif

THROTTLE_INL int32	ThreadThrottle()		THROTTLE_IMPL(return 0;)
THROTTLE_INL void	ThreadUnthrottle(int32)	THROTTLE_IMPL()

#undef THROTTLE_INL
#undef THROTTLE_IMPL

////////////////////////////////////////////////////////////////////////////////
struct FProfilerScope
{
	static OnScopeBeginFunc* OnScopeBegin;
	static OnScopeEndFunc* OnScopeEnd;

#if TRACE_PRIVATE_FULL_ENABLED
	FProfilerScope(const ANSICHAR* Label) { if (OnScopeBegin) { OnScopeBegin(Label); }}
	~FProfilerScope() { if(OnScopeEnd) { OnScopeEnd(); }}
#else
	FProfilerScope(const ANSICHAR* Label) {}
#endif
};
	
} // namespace Private
} // namespace Trace
} // namespace UE

#endif // TRACE_PRIVATE_MINIMAL_ENABLED
