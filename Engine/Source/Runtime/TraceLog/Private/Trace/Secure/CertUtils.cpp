// Copyright Epic Games, Inc. All Rights Reserved.
#include "CertUtils.h"
#include "Trace/Trace.h"
#include "Trace/Message.h"

#if UE_TRACE_MINIMAL_ENABLED && TRACE_PRIVATE_ALLOW_SECURE_TRACING 

#include "openssl/rand.h"
#include <cwchar>

namespace UE::Trace::Private
{

////////////////////////////////////////////////////////////////////////////////
void* Writer_MemoryAllocate(SIZE_T, uint32);
void Writer_MemoryFree(void*, uint32);

////////////////////////////////////////////////////////////////////////////////
bool GeneratePrivateKey(EVP_PKEY* PrivateKey)
{
	if (!PrivateKey)
	{
		return false;
	}

	struct FScopedBigNum
	{
		FScopedBigNum()
		{
			BigNumPtr = BN_new();
		}

		~FScopedBigNum()
		{
			if (BigNumPtr)
			{
				BN_free(BigNumPtr);
			}
		}

		BIGNUM* BigNumPtr = nullptr;
	};

	RSA* Rsa = RSA_new();
	if (!Rsa)
	{
		return false;
	}

	FScopedBigNum ScopedBigNum;

	if (!ScopedBigNum.BigNumPtr)
	{
		RSA_free(Rsa);
		return false;
	}

	if (!BN_set_word(ScopedBigNum.BigNumPtr, RSA_F4))
	{
		RSA_free(Rsa);
		return false;
	}

	constexpr int32 Bits = 2048;

	if (!RSA_generate_key_ex(Rsa, Bits, ScopedBigNum.BigNumPtr, nullptr))
	{
		RSA_free(Rsa);
		return false;
	}

	if (!EVP_PKEY_assign_RSA(PrivateKey, Rsa))
	{
		RSA_free(Rsa);
		return false;
	}

	return true;
}

////////////////////////////////////////////////////////////////////////////////
void AddSubjectNameEntryToCertificate(X509_NAME* SubjectName, const char* FiledType, const char* Value)
{
	if (SubjectName)
	{	
		constexpr int32 Len = -1; //Make OpenSSL calc len
		constexpr int32 Loc = -1;
		constexpr int32 Set = 0;
		X509_NAME_add_entry_by_txt(
			SubjectName, 
			reinterpret_cast<const char*>(FiledType), 
			MBSTRING_UTF8, 
			reinterpret_cast<const unsigned char*>(Value),
			Len,
			Loc, 
			Set
		);
	}
}

////////////////////////////////////////////////////////////////////////////////
bool GenerateCertificate(EVP_PKEY* PrivateKey, X509* x509, const FCertificateOptions& InCertOptions)
{
	if (!PrivateKey)
	{
		return false;
	}

	if (!x509)
	{
		return false;
	}

	uint32 SerialNumber;
	if (RAND_bytes(reinterpret_cast<unsigned char*>(&SerialNumber), sizeof(SerialNumber)) != 1)
	{
		UE_TRACE_MESSAGE_F(SecureSocketError, "[%s] Failed to generate serial number.", __func__);
		return false;
	}

	SerialNumber = SerialNumber ? SerialNumber : 1;

	if (!ASN1_INTEGER_set(X509_get_serialNumber(x509), SerialNumber))
	{
		UE_TRACE_MESSAGE_F(SecureSocketError, "[%s] Failed to generate assign number.", __func__);
		return false;
	}


	// Set the certificate version to 3 (by default it is set to 1). V3 current version of the X.509 standard 
	// X509_VERSION_3 is equals to 2
	constexpr long CertificateVersion3 = 2;
	if (!X509_set_version(x509, CertificateVersion3))
	{
		UE_TRACE_MESSAGE_F(SecureSocketError, "[%s] Failed to set the certificate version.", __func__);
		return false;
	}

	// This certificate is valid from now until the specified duration. By default, the certificate will only last 1 hs
	X509_gmtime_adj(X509_get_notBefore(x509), 0);
	X509_gmtime_adj(X509_get_notAfter(x509), InCertOptions.DurationInSeconds);

	// Set the public key for our certificate.
	if (!X509_set_pubkey(x509, PrivateKey))
	{
		return false;
	}

	X509_NAME* SubjectName = X509_get_subject_name(x509);
	if (!SubjectName)
	{
		UE_TRACE_MESSAGE_F(SecureSocketError, "[%s] Failed to get subject name.", __func__);
		return false;
	}

	AddSubjectNameEntryToCertificate(SubjectName, "C", InCertOptions.Country);
	AddSubjectNameEntryToCertificate(SubjectName, "O", InCertOptions.Organization);
	AddSubjectNameEntryToCertificate(SubjectName, "CN", InCertOptions.CommonName);

	if (!X509_set_issuer_name(x509, SubjectName))
	{
		UE_TRACE_MESSAGE_F(SecureSocketError, "[%s] Failed to set issuer name.", __func__);
		return false;
	}

	if (!X509_sign(x509, PrivateKey, EVP_sha256()))
	{
		UE_TRACE_MESSAGE_F(SecureSocketError, "[%s] Failed to sign generated certificate.", __func__);
		return false;
	}

	return true;
}

////////////////////////////////////////////////////////////////////////////////
bool X509ToDER(X509* x509, uint8** OutEncoded, int32& OutEncodedSize)
{
	if (!x509 || !OutEncoded)
	{
		UE_TRACE_MESSAGE_F(SecureSocketError, "[%s] Failed to encode x509 certificate. No ptr provided", __func__);
		return false;
	}

	const int32 ExpectedLength = i2d_X509(x509, nullptr);
	if (ExpectedLength < 0)
	{
		UE_TRACE_MESSAGE_F(SecureSocketError, "[%s] Failed to encode x509 certificate. Invalid estimated length", __func__);
		return false;
	}

	uint8* EncodedCertAlloc = (uint8*) Writer_MemoryAllocate(ExpectedLength, sizeof(void*));
	uint8* EncodedCert = EncodedCertAlloc;
	int32 FinalLength = i2d_X509(x509, &EncodedCert);
	if (FinalLength != ExpectedLength)
	{
		Writer_MemoryFree(EncodedCertAlloc, ExpectedLength);
		UE_TRACE_MESSAGE_F(SecureSocketError, "[%s] Failed to encode x509 certificate. Encode function returned an unexpected length | Expected [%d] | Result [%d]", __func__, ExpectedLength, FinalLength);
		return false;
	}

	*OutEncoded = EncodedCertAlloc;
	OutEncodedSize = FinalLength;
	return true;
}

////////////////////////////////////////////////////////////////////////////////
bool DERToX509(const uint8* EncodedCert, size_t EncodedSize, X509** OutCert)
{
	if (!EncodedCert || !EncodedSize || !OutCert)
	{
		UE_TRACE_MESSAGE_F(SecureSocketError, "[%s] Failed to decode x509 certificate. Invalid argument", __func__);
		return false;
	}

	X509* DecodedCert = d2i_X509(nullptr, &EncodedCert, EncodedSize);

	if (!DecodedCert)
	{
		UE_TRACE_MESSAGE_F(SecureSocketError, "[%s] Failed to decode encode x509 certificate.", __func__);
		return false;
	}

	*OutCert = DecodedCert;
	return true;
}

////////////////////////////////////////////////////////////////////////////////
bool CreateSelfSignedCert(const FCertificateOptions& InCertOptions, X509** OutCert, EVP_PKEY** OutPrivateKey)
{
	if (!OutCert || !OutPrivateKey)
	{
		return false;
	}
	EVP_PKEY* PrivateKey = EVP_PKEY_new();
	if (!GeneratePrivateKey(PrivateKey))
	{
		UE_TRACE_MESSAGE_F(SecureSocketError, "[%s] Failed to generate private key.", __func__);
		EVP_PKEY_free(PrivateKey);
		return false;
	}

	X509* x509 = X509_new();
	if (!GenerateCertificate(PrivateKey, x509, InCertOptions))
	{
		UE_TRACE_MESSAGE_F(SecureSocketError, "[%s] Failed to generate certificate.", __func__);

		EVP_PKEY_free(PrivateKey);
		X509_free(x509);
		return false;
	}

	*OutCert = x509;
	*OutPrivateKey = PrivateKey;
	return true;
}

} // namespace UE::Trace::Private

#endif // UE_TRACE_MINIMAL_ENABLED && TRACE_PRIVATE_ALLOW_SECURE_TRACING  
