// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Trace/Config.h"
#include "Trace/Platform.h"

#if UE_TRACE_MINIMAL_ENABLED && TRACE_PRIVATE_ALLOW_SECURE_TRACING 

#include "openssl/x509.h"

namespace UE::Trace::Private
{

//todo: Make required configuration option when secure connection is enabled
#define UE_TRACE_SECURESOCKET_DEFAULT_CMNNAME "Unreal Engine Trace"
#define UE_TRACE_SECURESOCKET_DEFAULT_ORG "Epic Games Inc"
#define UE_TRACE_SECURESOCKET_DEFAULT_COUNTRY "US"

/**
 * Generate a 2048-bit RSA key.
 *
 * @param PrivateKey The private key struct
 */
bool GeneratePrivateKey(EVP_PKEY* PrivateKey);

/** 
 * Options used to create an SSL Certificate. UTF-8 string
 * are supported.
 */
struct FCertificateOptions
{
	const char* CommonName = UE_TRACE_SECURESOCKET_DEFAULT_CMNNAME;
	const char* Organization = UE_TRACE_SECURESOCKET_DEFAULT_ORG;
	const char* Country = UE_TRACE_SECURESOCKET_DEFAULT_COUNTRY;
	long DurationInSeconds = 3600L;
};

/**
 * Generate self-signed x509 certificate.
 *
 * @param PrivateKey The private key struct
 * @param x509 The certificate struct
 * @return True if successful, false otherwise
 */
bool GenerateCertificate(EVP_PKEY* PrivateKey, X509* x509, const FCertificateOptions& InCertOptions);

/**
 * Encodes the provided certificate in DER format
 * @param x509 Certificate to Encode
 * @param OutEncoded A DER encode SSL Certificate
 * @param OutEncodedSize Size of the certificate
 * @return True if successful, false otherwise
 */
bool X509ToDER(X509* x509, uint8** OutEncoded, int32& OutEncodedSize);

/**
 * Decodes a DER formatted buffer into a x509 certificate instance
 * @param EncodedCert buffer containing the certificate encoded data
 * @param EncodedSize size of the byffer
 * @param OutCert A pointer to the generated X509 certificate instance
 * @return True if successful, false otherwise
 */
bool DERToX509(const uint8* EncodedCert, size_t EncodedSize, X509** OutCert);

/**
 * Creates a self signed certificate based on input options.
 * @param InCertOptions Options used to create the certificate
 * @param OutCert Generated certificate
 * @param OutPrivateKey Generated private key
 * @return True if successful, false otherwise
 */
bool CreateSelfSignedCert(const FCertificateOptions& InCertOptions, X509** OutCert, EVP_PKEY** OutPrivateKey);

} // namespace UE::Trace::Private

#endif // UE_TRACE_MINIMAL_ENABLED && TRACE_PRIVATE_ALLOW_SECURE_TRACING 
