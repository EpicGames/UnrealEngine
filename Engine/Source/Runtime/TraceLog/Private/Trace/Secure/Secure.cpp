// Copyright Epic Games, Inc. All Rights Reserved.
#include "Trace/Config.h"

#if UE_TRACE_MINIMAL_ENABLED && TRACE_PRIVATE_ALLOW_SECURE_TRACING 

#include "Trace/Detail/Atomic.h"
#include "Trace/Message.h"
#include "Trace/Secure/CertUtils.h"
#include "Trace/Trace.h"

#include <openssl/ssl.h>
#include <openssl/err.h>

/**
 * Enable extra verbose SSL debug logging
 */
#if !defined(TRACE_PRIVATE_SSL_DEBUG_LOGGING) 
	#define TRACE_PRIVATE_SSL_DEBUG_LOGGING 0
#endif


namespace UE {
namespace Trace {
namespace Private {

////////////////////////////////////////////////////////////////////////////////
void Writer_InternalInitialize();
bool Writer_SetPendingHandle(UPTRINT, IoWriteFunc, IoCloseFunc, IoUpdateFunc, IoIsReadyFunc);
void* Writer_MemoryAllocate(SIZE_T, uint32);
void Writer_MemoryFree(void*, uint32);
UPTRINT Writer_PackSendFlags(UPTRINT, uint32);

////////////////////////////////////////////////////////////////////////////////
#define UE_TRACE_LOG_IDENT(level, fmt, ident)\
	UE_TRACE_MESSAGE_F(level, "[S%p]" fmt, ident)

////////////////////////////////////////////////////////////////////////////////
#define UE_TRACE_LOG_IDENT_F(level, fmt, ident, ...)\
	UE_TRACE_MESSAGE_F(level, "[S%p]" fmt, ident, __VA_ARGS__)

////////////////////////////////////////////////////////////////////////////////
#define UE_TRACE_ERROR_IDENT(fmt, ident)\
	UE_TRACE_MESSAGE_F(SecureSocketError, "[S%p]" fmt, ident)

////////////////////////////////////////////////////////////////////////////////
#define UE_TRACE_ERROR_IDENT_F(fmt, ident, ...) \
	UE_TRACE_MESSAGE_F(SecureSocketError, "[S%p]" fmt, ident, __VA_ARGS__)

////////////////////////////////////////////////////////////////////////////////
#define UE_TRACE_OPENSSL_ERROR_IDENT(message, ident) \
	{ \
		while(int ErrorCode = ERR_get_error())\
		{\
			char SSLErrBuf[256]; \
			ERR_error_string_n(ErrorCode, SSLErrBuf, 256); \
			UE_TRACE_MESSAGE_F(SecureSocketError, "[S%p]" message ": %s", ident, SSLErrBuf);\
		}\
	}\

////////////////////////////////////////////////////////////////////////////////
#define UE_TRACE_OPENSSL_ERROR(message) \
	{ \
		while(int ErrorCode = ERR_get_error())\
		{\
			char SSLErrBuf[256]; \
			ERR_error_string_n(ErrorCode, SSLErrBuf, 256); \
			UE_TRACE_MESSAGE_F(SecureSocketError, message ": %s", SSLErrBuf);\
		}\
	}\

////////////////////////////////////////////////////////////////////////////////
class FSSLContext 
{
	public:
		FSSLContext() {}
		~FSSLContext() {}
		bool Initialize(const FSecureTraceSettings& Options);
		void Teardown();
		bool IsNonBlocking() const { return bNonBlocking; }

		bool bNonBlocking = false;
		uint16 Port;
		SSL_CTX* Context = nullptr;
		uint8* CertificateDER = nullptr;
		int32 CertificateDERSize = 0;
		ESecureTraceMode Mode;

	private:
		bool InitCertificates(const FSecureTraceSettings& Options);
		bool InitPSK(const FSecureTraceSettings& Options);
		void EnableDebugHooks();
		void DebugDumpDER(const char* Filename);

		X509* CertificateX509 = nullptr;
		EVP_PKEY* PrivateKey = nullptr;
};

////////////////////////////////////////////////////////////////////////////////
class FSSLConnection
{
	public:

		FSSLConnection(const FSSLContext& InContext, const FSecureTraceOptions& Options);
		FSSLConnection() = delete;
		FSSLConnection(FSSLConnection&) = delete;
		~FSSLConnection();

		bool Write(const void* Data, uint32 Size);
		void Close();
		bool Update();
		bool IsOpen() const;
		bool Listen(uint16 Port);
		static uint32 OnPSK(SSL *SSL, const char *Identity, unsigned char *PSK, unsigned int MaxPSKLen);
		
	private:
		enum EState : uint8 {
			None = 0,
			Closed,
			Listening,
			Handshake,
			Authenticating,
			Open,
			Error = 0xff
		};

		enum EMode : uint8 {
			Undefined		= 0,
			Send			= 1 << 1,
			Receive			= 1 << 2,
			Authenticated	= 1 << 3,
			NonBlocking		= 1 << 4,
		};

		bool CheckForTimeout(const char* Reason);

		uint8* AuthRecvBuffer;
		const uint8* AuthToken;
		char* Identity;
		uint32 IdentitySize;
		uint32 AuthSize;
		uint32 AuthRead;
		const FSSLContext& Context;
		SSL* Ssl;
		BIO* Bio;
		EMode Mode;
		EState State;
		uint64 WaitStartCycles;
		uint32 TimeoutMs;

		UPTRINT ListenHandle;
		UPTRINT SocketHandle;
};


////////////////////////////////////////////////////////////////////////////////
void FSSLContext::Teardown()
{
	Writer_MemoryFree(CertificateDER, CertificateDERSize);
	if (CertificateX509)
	{
		X509_free(CertificateX509);
	}
	if (PrivateKey)
	{
		EVP_PKEY_free(PrivateKey);
	}
	if (Context)
	{
		SSL_CTX_free(Context);
	}
}

bool FSSLContext::Initialize(const FSecureTraceSettings& Options)
{
	Mode = Options.Mode;
	Port = Options.Port;

	{
		uint64 OpenSSLOptions = OPENSSL_INIT_NO_ATEXIT;
		OPENSSL_init_ssl(OpenSSLOptions, nullptr);
	}

	auto* Method = TLSv1_2_server_method();
	Context = SSL_CTX_new(Method);

	if (!Context)
	{
		UE_TRACE_OPENSSL_ERROR("Failed to create SSL context");
		return false;
	}

	EnableDebugHooks();

	// const char* ModeNames[] = {"Self signed", "Certificate file", "Pre shared keys"};
	// UE_TRACE_MESSAGE_F(VeryVerbose, "SSLContext using mode %s", ModeNames[(uint8)Mode]);

	SSL_CTX_set_min_proto_version(Context, TLS1_2_VERSION);
	SSL_CTX_set_max_proto_version(Context, TLS1_2_VERSION);
	SSL_CTX_set_options(Context, SSL_OP_NO_TLSv1_3);

	// Setup certificates if those modes are requested
	if (Options.Mode == ESecureTraceMode::SelfSigned || Options.Mode == ESecureTraceMode::CertificateFile)
	{
		if (!InitCertificates(Options))
		{
			return false;
		}
	}
	else
	{
		if (!InitPSK(Options))
		{
			return false;
		}
	}

	return true;
}


bool FSSLContext::InitPSK(const FSecureTraceSettings& Options)
{
	SSL_CTX_set_options(Context, SSL_OP_CIPHER_SERVER_PREFERENCE);
	SSL_CTX_set_psk_server_callback(Context, &FSSLConnection::OnPSK);

	static const char* CipherList = "ECDHE-PSK-CHACHA20-POLY1305";

	if (!SSL_CTX_set_cipher_list(Context, CipherList))
	{
		UE_TRACE_OPENSSL_ERROR("Failed to set cipher suites");
		return false;
	}
	
	static const char* GroupLists = "P-384:P-256";

	if (!SSL_CTX_set1_groups_list(Context, GroupLists))
	{
		UE_TRACE_OPENSSL_ERROR("Failed to set group list");
		return false;
	}

	// UE_TRACE_MESSAGE_F(VeryVerbose, "Cipher list: '%s', groups list: '%s'", CipherList, GroupLists);
	return true;
}

bool FSSLContext::InitCertificates(const FSecureTraceSettings& Options)
{
	// Create certificate or use provided
	if (Options.Mode == ESecureTraceMode::SelfSigned)
	{
		// Create a certificate with the default named fields 
		FCertificateOptions CertCreationOptions;
		if (!CreateSelfSignedCert(CertCreationOptions, &CertificateX509, &PrivateKey))
		{
			return false;
		}

		if (!SSL_CTX_use_certificate(Context, CertificateX509))
		{
			UE_TRACE_OPENSSL_ERROR("Failed to load SSL certificate.");
			return false;
		}

		if (!SSL_CTX_use_PrivateKey(Context, PrivateKey))
		{
			UE_TRACE_OPENSSL_ERROR("Failed to load private key.");
			return false;
		}

		if (!X509ToDER(CertificateX509, &CertificateDER, CertificateDERSize))
		{
			UE_TRACE_OPENSSL_ERROR("Failed to decode certificate");
			return false;
		}

		// Uncomment to dump the binary into a file
		// DebugDumpDER("ServerCertificate.der");
	}
	else if (Options.Mode == ESecureTraceMode::CertificateFile)
	{
		if (!SSL_CTX_use_certificate_file(Context, Options.CertificatePath, SSL_FILETYPE_PEM))
		{
			// Don't log filepath to avoid leaking location
			UE_TRACE_OPENSSL_ERROR("Failed to load SSL certificate from file.");
			return false;
		}

		if (!SSL_CTX_use_PrivateKey_file(Context, Options.PrivateKeyPath, SSL_FILETYPE_PEM))
		{
			// Don't log filepath to avoid leaking location
			UE_TRACE_OPENSSL_ERROR("Failed to load private key from file.");
			return false;
		}
	}

	// Verify we have matching certificate and key
	if (!SSL_CTX_check_private_key(Context))
	{
		UE_TRACE_OPENSSL_ERROR("Could not verify private key");
		return false;
	}

	return true;
}

void FSSLContext::EnableDebugHooks()
{
#if TRACE_PRIVATE_SSL_DEBUG_LOGGING 
	SSL_load_error_strings();
	ERR_load_BIO_strings();
	OpenSSL_add_all_algorithms();

	SSL_CTX_set_info_callback(
	    Context,
	    [](const SSL *Ssl, int Type, int Value)
	    {
		    const char *Direction = (Type & SSL_CB_READ) ? "READ" : "WRITE";

		    // Check if the current state is Server processing Client Hello (after read)
			OSSL_HANDSHAKE_STATE State = SSL_get_state(Ssl);
		    if (State == TLS_ST_SR_CLNT_HELLO)
		    {
			    // Check if the Client Hello has been read (e.g., state > SSL_ST_BEFORE)
			    if (SSL_get_current_cipher(Ssl) == NULL) // Before a cipher is chosen
			    {
				    // --- Log Client's Offered Ciphers ---
				    STACK_OF(SSL_CIPHER) *ciphers = SSL_get_client_ciphers(Ssl);

				    // Check if the stack is available (only available after Client Hello is processed)
				    if (ciphers != NULL && (Type & SSL_CB_EXIT))
				    {
					    UE_TRACE_MESSAGE(VeryVerbose, "[TLS_INFO]|CLIENT_OFFERS|Ciphers offered by client:");

					    for (int i = 0; i < sk_SSL_CIPHER_num(ciphers); i++)
					    {
						    const SSL_CIPHER *cipher = sk_SSL_CIPHER_value(ciphers, i);

						    // Print the long name of the offered cipher
						    UE_TRACE_MESSAGE_F(VeryVerbose, "[TLS_INFO]|OFFERED_CIPHER|%d: %s", i + 1,
						                       SSL_CIPHER_get_name(cipher));
					    }
				    }
			    }
		    }

		    // --- State and Handshake Events ---
		    if (Type & (SSL_CB_LOOP | SSL_CB_EXIT))
		    {
			    // Log the state transition
			    const char* StateStr = SSL_state_string_long(Ssl);
			    // The message is constructed to include the state string.
			    UE_TRACE_MESSAGE_F(VeryVerbose, "[TLS_INFO]|Dir[%s]|Event[STATE_CHANGE]|State[%s]|Result[%s]", Direction,
			                       (StateStr ? StateStr : "N/A"), (Type & SSL_CB_EXIT) ? "EXIT" : "LOOP");
			    return;
		    }

		    switch (Type)
		    {
		    case SSL_CB_HANDSHAKE_START:
			    // Log the start of the handshake
			    UE_TRACE_MESSAGE_F(VeryVerbose, "[TLS_INFO]|Dir[%s]|Event[HANDSHAKE_START]|Protocol[%s]", Direction,
			                       SSL_get_version(Ssl) // Protocol version used for the Client/Server Hello
			    );
			    break;

		    case SSL_CB_HANDSHAKE_DONE:
			    UE_TRACE_MESSAGE_F(VeryVerbose, "[TLS_INFO]|Dir[%s]|Event[HANDSHAKE_DONE]|Protocol[%s]|Cipher[%s]", Direction,
			                       SSL_get_version(Ssl),
			                       SSL_get_cipher_name(Ssl) // The negotiated cipher 
			    );
			    break;

		    case SSL_CB_ALERT:
		    {
			    // Log a received or sent alert
			    const char* TypeStr = SSL_alert_type_string_long(Value);
			    const char* DescStr = SSL_alert_desc_string_long(Value);

			    // Log the alert with both the name and the raw code
			    UE_TRACE_MESSAGE_F(VeryVerbose, "[TLS_INFO]|Dir[%s]|Event[ALERT]|Type[%s]|Desc[%s] (Code: %d)", Direction,
			                       (TypeStr ? TypeStr : "N/A"), (DescStr ? DescStr : "N/A"), Value);
			    break;
		    }

		    default:
			    // Log any other unhandled event type
			    UE_TRACE_MESSAGE_F(VeryVerbose, "[TLS_INFO]|Dir[%s]|Event[UNKNOWN]|Type[%d]|Value[%d]", Direction, Type, Value);
			    break;
		    }
	    });

	SSL_CTX_set_msg_callback(Context, [](int write_p, int version, int content_type, const void *buf, size_t len, SSL *ssl, void *arg) {
		UE_TRACE_MESSAGE_F(VeryVerbose, 
			"TLS Traffic [%s]|Type [%d]|Length [%zu]", 
			write_p ? "In" : "Out", 
			content_type, 
			len
		);
	});
#endif // WITH_SSL_DEBUG_LOGGING
}

void FSSLContext::DebugDumpDER(const char* Filename)
{
	auto FileHandle = FileOpen(Filename);
	if (FileHandle)
	{
		const bool bSuccess = IoWrite(FileHandle, CertificateDER, CertificateDERSize);
		if (!bSuccess)
		{
			UE_TRACE_MESSAGE_F(SecureSocketError, "Failed to dump server certificate to '%s'.", Filename);
		}
		else
		{
			UE_TRACE_MESSAGE_F(Verbose, "Successfully dumped certificate to %s (%d bytes)", Filename, CertificateDERSize);
		}
		IoClose(FileHandle);
	}
}

////////////////////////////////////////////////////////////////////////////////

FSSLConnection::FSSLConnection(const FSSLContext& InContext, const FSecureTraceOptions& Options)
	: AuthRecvBuffer(nullptr)
	, AuthToken(nullptr)
	, Identity(nullptr)
	, AuthRead(0)
	, Context(InContext)
	, Ssl(nullptr)
	, Bio(nullptr)
	, State(EState::Closed)
	, WaitStartCycles(0)
	, TimeoutMs(Options.TimeoutMs)
	, ListenHandle(0)
	, SocketHandle(0)
{
	const bool bAdditionalAuthentication = Options.AuthorizationTokenSize > 0 && InContext.Mode != ESecureTraceMode::PreSharedKeys;
	Mode = (EMode) (
		Send |
		(Options.bCanReceive ? Receive : 0u) |
		(bAdditionalAuthentication ? Authenticated : 0u) |
		(InContext.IsNonBlocking() ? NonBlocking : 0u)
	);

	AuthSize = Options.AuthorizationTokenSize;
	AuthToken = Options.AuthorizationToken;

	if (Mode & Authenticated)
	{
		AuthRecvBuffer = (uint8*) Writer_MemoryAllocate(AuthSize, 8);
	}

	const uint32 IdentityLen = Options.Identity ? (strlen(Options.Identity) + 1) : 0;
	IdentitySize = IdentityLen * sizeof(char);
	if (IdentitySize)
	{
		Identity = (char*) Writer_MemoryAllocate(IdentitySize, 8);
		strncpy(Identity, Options.Identity, IdentityLen);
		Identity[IdentityLen-1] = '\0';
	}
}

FSSLConnection::~FSSLConnection()
{
	if (Ssl)
	{
		SSL_free(Ssl);
	}
	if (AuthRecvBuffer)
	{
		Writer_MemoryFree(AuthRecvBuffer, AuthSize);
	}
	if (Identity)
	{
		Writer_MemoryFree(Identity, IdentitySize);
	}
	if (ListenHandle)
	{
		IoClose(ListenHandle);
	}
	if (SocketHandle)
	{
		IoClose(SocketHandle);
	}
}

bool FSSLConnection::Listen(uint16 ListenPort)
{
	ListenHandle = TcpSocketListen(ListenPort);
	if (!ListenHandle)
	{
		UE_TRACE_ERROR_IDENT_F("Failed to create listen socket on port %u", this, ListenPort);
		return false;
	}
	UE_TRACE_LOG_IDENT_F(Log, "Started listening on port %u.", this, ListenPort);
	AtomicStoreRelaxed(&State, EState::Listening);
	return true;
}

bool FSSLConnection::Update()
{
	switch(State)
	{
		case EState::Error:
			return false;

		case EState::Listening:
			{
				UPTRINT AcceptedHandle;
				int Return = TcpSocketAccept(ListenHandle, AcceptedHandle, true);
				if (Return <= 0)
				{
					if (Return == -1)
					{
						UE_TRACE_ERROR_IDENT("Failed to accept connection.", this);
						AtomicStoreRelaxed(&State, EState::Error); 
						return false;
					}

					if (CheckForTimeout("Timeout waiting for connection."))
					{
						return false;
					}

					return true; // No connection yet
				}

				// Log accepting connections
				char AcceptingIp[256];
				const bool bCouldResolveName = TcpSocketPeerName(AcceptedHandle, AcceptingIp, sizeof(AcceptingIp));
				UE_TRACE_LOG_IDENT_F(Log, "Accepting connection from %s.", this, bCouldResolveName ? AcceptingIp : "unknown");

				SocketHandle = AcceptedHandle;

				// Create SSL state for this connection and setup buffered io
				Ssl = SSL_new(Context.Context);
				Bio = BIO_new_socket(TcpSocketNative(SocketHandle), 0); 

				if (!Ssl || !Bio)
				{
					UE_TRACE_OPENSSL_ERROR_IDENT("Failed to create SSL/buffered io for connection", this);
					AtomicStoreRelaxed(&State, EState::Error); 
					SSL_free(Ssl);
					Ssl = nullptr;
					BIO_free(Bio); // Need to manually free the bio since it's not yet attached.
					Bio = nullptr;
					Close();
					return false;
				}

				BIO_set_nbio(Bio, 1);
				SSL_set_bio(Ssl, Bio, Bio);

				if (Context.Mode == ESecureTraceMode::PreSharedKeys)
				{
					SSL_set_app_data(Ssl, this);
				}

				// Close listening socket
				IoClose(ListenHandle);
				ListenHandle = 0;

				// Reset timeout for next stage
				WaitStartCycles = 0;

				AtomicStoreRelaxed(&State, EState::Handshake);
			}
			break;

		case EState::Handshake:
			{
				const int Result = SSL_accept(Ssl);
				if (Result <= 0)
				{
					int SSLError = SSL_get_error(Ssl, Result);
					if (SSLError == SSL_ERROR_WANT_READ || SSLError == SSL_ERROR_WANT_WRITE)
					{
						if (CheckForTimeout("Timeout waiting for handshake"))
						{
							return false;
						}
						return true;
					}
					UE_TRACE_OPENSSL_ERROR_IDENT("Failed SSL handshake", this);
					AtomicStoreRelaxed(&State, EState::Error); 
					Close();
					return false;
				}

				WaitStartCycles = 0;

				UE_TRACE_LOG_IDENT(Verbose, "SSL handshake completeted.", this);
				if ((Mode & Authenticated) != 0)
				{
					AtomicStoreRelaxed(&State, EState::Authenticating);
				}
				else
				{
					AtomicStoreRelaxed(&State, EState::Open);
				}
			}
			break;

		case EState::Authenticating:
			{
				const int BytesRead = SSL_read(Ssl, AuthRecvBuffer + AuthRead, AuthSize - AuthRead);
				if (BytesRead < 0)
				{
					const int SslError = SSL_get_error(Ssl, BytesRead);
					if (SslError == SSL_ERROR_WANT_READ || SslError == SSL_ERROR_WANT_WRITE)
					{
						if (CheckForTimeout("Timeout waiting for authentication"))
						{
							return false;
						}
						return true;
					}
					UE_TRACE_OPENSSL_ERROR_IDENT("Transfer error", this);
					AtomicStoreRelaxed(&State, EState::Error); 
					Close();
					return false;
				}
				else if (BytesRead == 0)
				{
					UE_TRACE_LOG_IDENT(Log, "Connection closed by peer", this);
					AtomicStoreRelaxed(&State, EState::Closed);
					Close();
					return false;
				}
				AuthRead += BytesRead;
				if (AuthRead == AuthSize)
				{
					if (memcmp(AuthRecvBuffer, AuthToken, AuthSize) == 0)
					{
						UE_TRACE_LOG_IDENT(Log, "Authorisation token accepted.", this);
						AtomicStoreRelaxed(&State, EState::Open);

						// Disable non-blocking mode. This cannot happen if there is
						// pending data, which we don't expect from the peer at this point.
						if (SSL_pending(Ssl))
						{
							UE_TRACE_ERROR_IDENT("Unexpected pending data in connection", this);
							return false;
						}

						if (!TcpSocketSetNonBlocking(SocketHandle, false)) 
						{
							UE_TRACE_ERROR_IDENT("Failed to reset non-blocking mode on socket", this);
							return false;
						}

						// BIO control macros sometimes return the 0 event though
						// no actual error occurred, so also check error stack
						int BioRes = BIO_set_nbio(Bio, 0);
						if (BioRes != 1 && ERR_peek_error() != 0) 
						{
							UE_TRACE_OPENSSL_ERROR_IDENT("Failed to disable non-blocking mode on bio", this);
							return false;
						}
						
						return true;
					}
					else
					{
						UE_TRACE_ERROR_IDENT("Authorisation token rejected.", this);
						Close();
						return false;
					}
				}
			}
			break;
	}

	return true;
}

bool FSSLConnection::Write(const void* Data, uint32 Size)
{
	if (!IsOpen()) 
	{
		return false;
	}
	
	const int BytesWritten = SSL_write(Ssl, Data, Size);
	if (BytesWritten != Size)
	{
		UE_TRACE_OPENSSL_ERROR_IDENT("Write error", this);
	}
	return BytesWritten == Size;
}

void FSSLConnection::Close()
{
	EState Previous = AtomicExchangeRelease(&State, EState::Closed);
	if (Previous != EState::Closed)
	{
		UE_TRACE_LOG_IDENT(Log, "Connection closed", this);
		IoClose(SocketHandle);
		SocketHandle = 0;
	}
}

inline bool FSSLConnection::IsOpen() const
{
	return AtomicLoadRelaxed<uint8>((uint8*)&State) == EState::Open;
}

bool FSSLConnection::CheckForTimeout(const char* Reason)
{
	if (!TimeoutMs)
	{
		return false;
	}
	const uint64 Ts = TimeGetRelativeTimestamp();
	WaitStartCycles = WaitStartCycles ? WaitStartCycles : Ts;
	uint32 WaitedMs = uint32(1000.f*(float(Ts - WaitStartCycles) / float(TimeGetFrequency())));
	if (WaitedMs > TimeoutMs)
	{
		UE_TRACE_ERROR_IDENT_F(" %s", this, Reason);
		AtomicStoreRelaxed(&State, EState::Error); 
		return true;
	}
	return false;
}

uint32 FSSLConnection::OnPSK(SSL* Ssl, const char* Identity, unsigned char* PSK, unsigned int MaxPSKLen)
{
	FSSLConnection* Self = (FSSLConnection*)SSL_get_app_data(Ssl);
	if (!Self)
	{
		UE_TRACE_MESSAGE(SecureSocketError, "Unable to get app data.");
		return 0;
	}

	if (MaxPSKLen < Self->AuthSize)
	{
		UE_TRACE_OPENSSL_ERROR_IDENT("Target PSK buffer too small.", Self);
		return 0;
	}

	if (!Identity)
	{
		UE_TRACE_ERROR_IDENT("Peer must provide identity.", Self);
		return 0;
	}

	if (strcmp(Identity, Self->Identity) != 0)
	{
		UE_TRACE_ERROR_IDENT_F("Unknown identity '%s'", Self, Identity);
		return 0;
	}

	// UE_TRACE_MESSAGE_F(Verbose, "Retrieving pre-shared key for identity '%s'", Identity);
	memcpy(PSK, Self->AuthToken, Self->AuthSize);

	return Self->AuthSize;
}


////////////////////////////////////////////////////////////////////////////////
class FSecure 
{
	public:
		static bool Initialize(const FSecureTraceSettings& Settings);
		static bool IsInitialized();
		static bool GetSelfSignedCert(uint8 **OutPtr, uint32 *OutSize);
		static void Teardown();
		static FSSLConnection* CreateConnection(const FSecureTraceOptions& Options);
		static void RemoveConnection(FSSLConnection* Connection);

	private:
		static FSSLContext SSLContext;	
		static FSSLConnection* SSLConnection;
		static bool bIsInitialized;
};

bool FSecure::Initialize(const FSecureTraceSettings& Settings)
{
	static bool bInitializedInternal = [&Settings]() -> bool {
		bool bSuccess = SSLContext.Initialize(Settings);
		if (!bSuccess)
		{
			UE_TRACE_MESSAGE(SecureInitError, "Failed to initialize secure connections");
		}
		else
		{
			bIsInitialized = true;
		}
		return true;
	}();
	return bIsInitialized;
}

bool FSecure::IsInitialized()
{
	return bIsInitialized;
}

bool FSecure::GetSelfSignedCert(uint8 **OutPtr, uint32 *OutSize)
{
	if (SSLContext.CertificateDER)
	{
		*OutPtr = SSLContext.CertificateDER;
		*OutSize = SSLContext.CertificateDERSize;
		return true;
	}
	return false;
}

void FSecure::Teardown()
{
	if (SSLConnection)
	{
		SSLConnection->Close();
		SSLConnection->~FSSLConnection();
		Writer_MemoryFree(SSLConnection, sizeof(FSSLConnection));
		SSLConnection = nullptr;
	}
	SSLContext.Teardown();
}

FSSLConnection* FSecure::CreateConnection(const FSecureTraceOptions& Options)
{
	if (SSLConnection)
	{
		UE_TRACE_MESSAGE(ConnectError, "A secure connection already exists"); 
		return nullptr;
	}

	if (SSLContext.Mode == ESecureTraceMode::PreSharedKeys &&
	    (Options.AuthorizationTokenSize == 0 || Options.AuthorizationToken == nullptr))
	{
		UE_TRACE_MESSAGE(ConnectError, "An authorization token must be provided in PreSharedKeys mode"); 
		return nullptr;
	}

	if (SSLContext.Mode == ESecureTraceMode::PreSharedKeys && Options.Identity == nullptr)
	{
		UE_TRACE_MESSAGE(ConnectError, "An identity must be provided in PreSharedKeys mode"); 
		return nullptr;
	}

	// For now we only support one secure connection
	SSLConnection = (FSSLConnection*) Writer_MemoryAllocate(sizeof(FSSLConnection), alignof(FSSLConnection));
	SSLConnection = new(SSLConnection) FSSLConnection(SSLContext, Options); 
	if (!SSLConnection->Listen(SSLContext.Port))
	{
		RemoveConnection(SSLConnection);
		return nullptr;
	}
	return SSLConnection;
}

void FSecure::RemoveConnection(FSSLConnection* Connection)
{
	if (SSLConnection)
	{
		SSLConnection->Close();
		SSLConnection->~FSSLConnection();
		Writer_MemoryFree(SSLConnection, sizeof(FSSLConnection));
		SSLConnection = nullptr;
	}
}

FSSLContext FSecure::SSLContext;
FSSLConnection* FSecure::SSLConnection;
bool FSecure::bIsInitialized;

////////////////////////////////////////////////////////////////////////////////
bool SecureWrite(UPTRINT Handle, const void* Data, uint32 Size)
{
	if (!Handle)
	{
		return false;
	}
	FSSLConnection* Connection = (FSSLConnection*) Handle;
	return Connection->Write(Data, Size);
}

////////////////////////////////////////////////////////////////////////////////
void SecureClose(UPTRINT Handle)
{
	if (!Handle)
	{
		return;
	}
	FSSLConnection* Connection = (FSSLConnection*) Handle;
	Connection->Close();
	FSecure::RemoveConnection(Connection);
}

////////////////////////////////////////////////////////////////////////////////
bool SecureUpdate(UPTRINT Handle)
{
	if (!Handle)
	{
		UE_TRACE_MESSAGE(SecureInitError, "No handle, closing connection");
		return false;
	}
	FSSLConnection* Connection = (FSSLConnection*) Handle;
	return Connection->Update();
}

////////////////////////////////////////////////////////////////////////////////
bool SecureIsReady(UPTRINT Handle)
{
	if (!Handle)
	{
		return false;
	}
	FSSLConnection* Connection = (FSSLConnection*) Handle;
	return Connection->IsOpen();
}
////////////////////////////////////////////////////////////////////////////////
bool Writer_SecureInit(const FSecureTraceSettings& Settings)
{
	// Verify arguments
	if (Settings.Mode == ESecureTraceMode::CertificateFile && (!Settings.CertificatePath || !Settings.PrivateKeyPath))
	{
		UE_TRACE_MESSAGE(SecureInitError, "CertificatePath and PrivateKeyPath needs to be set when using CertificateFile mode.");
		return false;
	}

	return FSecure::Initialize(Settings);
}

////////////////////////////////////////////////////////////////////////////////
bool Writer_SecureIsInit()
{
	return FSecure::IsInitialized();
}

bool Writer_SecureGetSelfSignedCert(uint8 **OutPtr, uint32 *OutSize)
{
	return FSecure::GetSelfSignedCert(OutPtr, OutSize);
}

////////////////////////////////////////////////////////////////////////////////
bool Writer_SecureSendTo(const FSecureTraceOptions& Options, uint16 Flags)
{
	if (!FSecure::IsInitialized())
	{
		UE_TRACE_MESSAGE(ConnectError, "Secure connection has not been initialized, failed to create connection.");
		return false;
	}
	FSSLConnection* NewConnection = FSecure::CreateConnection(Options);
	if (!NewConnection)
	{
		UE_TRACE_MESSAGE(ConnectError, "Failed to create secure connection.");
		return false;
	}

	UPTRINT DataHandle = (UPTRINT) NewConnection;
	DataHandle = Writer_PackSendFlags(DataHandle, Flags);

	return Writer_SetPendingHandle(
		DataHandle, 
		SecureWrite, 
		SecureClose, 
		SecureUpdate, 
		SecureIsReady
	);
}

} //namespace Private

////////////////////////////////////////////////////////////////////////////////
bool SecureIsInitialized()
{
	return Private::Writer_SecureIsInit();
}

////////////////////////////////////////////////////////////////////////////////
bool SecureInitialize(const FSecureTraceSettings& Settings)
{
	return Private::Writer_SecureInit(Settings);
}

bool SecureGetSelfSignedCertificate(uint8 **OutPtr, uint32 *OutSize)
{
	return Private::Writer_SecureGetSelfSignedCert(OutPtr, OutSize);
}

////////////////////////////////////////////////////////////////////////////////
bool SecureSendTo(const FSecureTraceOptions& Options, uint16 Flags)
{
	return Private::Writer_SecureSendTo(Options, Flags);
}

} //namespace Trace
} //namespace UE

#endif // UE_TRACE_MINIMAL_ENABLED
