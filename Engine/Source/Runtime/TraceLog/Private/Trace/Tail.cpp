// Copyright Epic Games, Inc. All Rights Reserved.

#include "Trace/Config.h"

#if TRACE_PRIVATE_MINIMAL_ENABLED

#include "Platform.h"
#include "Trace/Detail/Atomic.h"
#include "Trace/Detail/Protocol.h"
#include "Trace/Detail/Transport.h"

#include <string.h>
#include <type_traits>
#include <initializer_list>

namespace UE {
namespace Trace {
namespace Private {

// This here to aid future maintenance of a trace's transport packets.
static_assert(ETransport::Active == ETransport::TidPacketSync, "Tail-tracing is transport aware");

////////////////////////////////////////////////////////////////////////////////
uint32		GetEncodeMaxSize(uint32);
int32		EncodeInstr(const void*, int32, void*, int32);
void*		Writer_MemoryAllocate(SIZE_T, uint32);
void		Writer_MemoryFree(void*, uint32);
void		Writer_SendData(uint32, uint8* __restrict, uint32);
void		Writer_SendDataRaw(const void*, uint32);

extern FStatistics GTraceStatistics;	

#if UE_TRACE_PACKET_VERIFICATION	
extern uint64 GPacketSerial;
#endif
	
////////////////////////////////////////////////////////////////////////////////
// ** See the bottom of this file for an explanation of FPacketRing
class FPacketRing
{
public:
	struct FRange
	{
		const void*	Data;
		uint32		Size;
	};

	void			Initialize(uint32 InSize);
	void			Shutdown();
	void			Reset();
	uint32			GetSize() const;
	bool			IsActive() const;
	FRange			GetBackPackets() const;
	FRange			GetFrontPackets() const;
	template <typename CallbackType>
	void			IterateRanges(CallbackType&& Callback, uint32 MaxSize);
	template <typename PacketType>
	PacketType*		Append(uint32 InSize);
	template <typename PacketType>
	void			BackUp(uint32 InSize);
	void			BackUp(uint32 InSize);

private:
	FTidPacketBase*	AppendImpl(uint32 InSize);
	uint8*			Data;
	uint32			Size;
	uint32			Cursor;
	uint32			Left;
	uint32			Right;
};

// TraceLog must be ready after value-init so that it can be used before
// dynamic-init. Thus statically-scoped objects cannot have [con|des]tructors.
static_assert(std::is_trivial<FPacketRing>(), "FPacketRing must be trivial");

////////////////////////////////////////////////////////////////////////////////
void FPacketRing::Initialize(uint32 InSize)
{
	Data = (uint8*)Writer_MemoryAllocate(InSize, 16);
#if TRACE_PRIVATE_STATISTICS
	AtomicAddRelaxed(&GTraceStatistics.FixedBufferAllocated, uint32(InSize));
#endif
	Size = InSize;
	Reset();
}

////////////////////////////////////////////////////////////////////////////////
void FPacketRing::Shutdown()
{
	Writer_MemoryFree(Data, Size);
#if TRACE_PRIVATE_STATISTICS
	AtomicSubRelaxed(&GTraceStatistics.FixedBufferAllocated, uint32(Size));
#endif
	Data = nullptr;
}

////////////////////////////////////////////////////////////////////////////////
void FPacketRing::Reset()
{
	Cursor = 0;
    Left = Right = Size;
}

////////////////////////////////////////////////////////////////////////////////
uint32 FPacketRing::GetSize() const
{
	return Size;
}

////////////////////////////////////////////////////////////////////////////////
bool FPacketRing::IsActive() const
{
	return Data != nullptr;
}

////////////////////////////////////////////////////////////////////////////////
FPacketRing::FRange FPacketRing::GetBackPackets() const
{
	return { Data + Left, Right - Left };
}

////////////////////////////////////////////////////////////////////////////////
FPacketRing::FRange FPacketRing::GetFrontPackets() const
{
	return { Data, Cursor };
}

////////////////////////////////////////////////////////////////////////////////
template <typename CallbackType>
void FPacketRing::IterateRanges(CallbackType&& Callback, uint32 MaxSize)
{
	FPacketRing::FRange Ranges[] = { GetBackPackets(), GetFrontPackets() };

	int32 BytesToSkip = 0;
	if (MaxSize != 0 && MaxSize < INT32_MAX)
	{
		const uint32 TotalSize = Ranges[0].Size + Ranges[1].Size;
		const uint32 TotalBytesToSkip = MaxSize > TotalSize ? 0 : TotalSize - MaxSize;
		BytesToSkip = TotalBytesToSkip;
	}

	for (const auto& Range : Ranges)
	{
		if (Range.Size == 0)
		{
			continue;
		}
		uint32 RangeSkipped = 0;
		uint8* RangeOffset = (uint8*) Range.Data;
		while (BytesToSkip > 0 && RangeOffset < ((uint8*)Range.Data + Range.Size))
		{
			// We need to skip over entire packages
			FTidPacketBase* Packet = (FTidPacketBase*) RangeOffset;
			BytesToSkip -= Packet->PacketSize;
			RangeSkipped += Packet->PacketSize;
			RangeOffset = RangeOffset + Packet->PacketSize;
		} 

		FPacketRing::FRange RangeView = {RangeOffset, Range.Size - RangeSkipped};
		Callback(RangeView);
	}
}

////////////////////////////////////////////////////////////////////////////////
template <typename PacketType>
PacketType* FPacketRing::Append(uint32 InSize)
{
	FTidPacketBase* Ptr = AppendImpl(InSize + sizeof(PacketType));
	return static_cast<PacketType*>(Ptr);
}

////////////////////////////////////////////////////////////////////////////////
template <typename PacketType>
void FPacketRing::BackUp(uint32 InSize)
{
	Cursor -= (InSize + sizeof(PacketType));
}

////////////////////////////////////////////////////////////////////////////////
void FPacketRing::BackUp(uint32 InSize)
{
	Cursor -= InSize;
}

////////////////////////////////////////////////////////////////////////////////
FTidPacketBase* FPacketRing::AppendImpl(uint32 InSize)
{
	// ** See the bottom of this file for an explanation of the logic here.

	// Too big to fit in the buffer? It is not possible to maintain consisntency
	// past this point as some of the data would be truncated. So we drop all
	// known data and start afresh.
	if (UNLIKELY(InSize > Size))
	{
		Reset();
		return nullptr;
	}

	uint32 NextCursor = Cursor + InSize;

	// Run off the end of the buffer?
	if (UNLIKELY(NextCursor > Size))
	{
		Left = 0;
		Right = Cursor;
		Cursor = 0;
		NextCursor = InSize;
	}

	// Discard old packets until there is space for the new one
	while (true)
	{
		// Does [Cursor, NextCursor) no longer overlap [Left, Right)?
		if (LIKELY(Left >= NextCursor))
		{
			break;
		}

		// Is [Left, Right) now empty?
		if (UNLIKELY(Left >= Right))
		{
			break;
		}

		const auto* TidPacket = (const FTidPacketBase*)(Data + Left);
		Left += TidPacket->PacketSize;
	}

	// Drop a packet from left.
	auto* TidPacket = (FTidPacketBase*)(Data + Cursor);
	TidPacket->PacketSize = uint16(InSize);

	Cursor = NextCursor;
	return TidPacket;
}



////////////////////////////////////////////////////////////////////////////////
static FPacketRing GPacketRing; // = {};

////////////////////////////////////////////////////////////////////////////////
void Writer_TailAppend(uint32 ThreadId, uint8* __restrict Data, uint32 Size)
{
	FProfilerScope _(__func__);

	// We expect supplied thread id to not contain any flags
	ThreadId &= FTidPacketBase::ThreadIdMask;
	
	// Perhaps tail tracing is disabled?
	if (!GPacketRing.IsActive())
	{
		return Writer_SendData(ThreadId, Data, Size);
	}

	// Smaller buffers usually aren't redundant enough to benefit from being
	// compressed. They often end up being larger.
	if (Size > 384)
	{
		uint32 EncodeMaxSize = GetEncodeMaxSize(Size);
		FTidPacketEncoded* Packet = GPacketRing.Append<FTidPacketEncoded>(EncodeMaxSize);
		if (Packet == nullptr)
		{
			// FPacketRing::AppendImpl resets if the packet is too large,
			return Writer_SendData(ThreadId, Data, Size);
		}
		Packet->ThreadId = uint16(ThreadId);
		Packet->ThreadId |= FTidPacketBase::EncodedMarker;
#if UE_TRACE_PACKET_VERIFICATION
		Packet->ThreadId |= FTidPacketBase::Verification;
#endif
		Packet->DecodedSize = uint16(Size);

		int32 EncodedSize = EncodeInstr(Data, Size, Packet->Data, EncodeMaxSize);
		if (EncodedSize > 0)
		{
			// Reclaim buffer space (worst case size - actual size)
			uint32 BackUp = EncodeMaxSize - EncodedSize;
			GPacketRing.BackUp(BackUp);
			Packet->PacketSize -= uint16(BackUp);

			Writer_SendDataRaw(Packet, Packet->PacketSize);
#if UE_TRACE_PACKET_VERIFICATION
			const uint64 Serial = GPacketSerial++;
			Writer_SendDataRaw(&Serial, sizeof(uint64));
#endif
			return;
		}
		else
		{
			// Compression failed, back out of entire packet
			GPacketRing.BackUp<FTidPacketEncoded>(EncodeMaxSize);
		}
	}

	FTidPacket* Packet = GPacketRing.Append<FTidPacket>(Size);
	if (Packet == nullptr)
	{
		// FPacketRing::AppendImpl resets if the packet is too large,
		return Writer_SendData(ThreadId, Data, Size);
	}
	Packet->ThreadId = uint16(ThreadId);
#if UE_TRACE_PACKET_VERIFICATION
	Packet->ThreadId |= FTidPacketBase::Verification;
#endif
	::memcpy(Packet->Data, Data, Size);

	Writer_SendDataRaw(Packet, Packet->PacketSize);
#if UE_TRACE_PACKET_VERIFICATION
	const uint64 Serial = GPacketSerial++;
	Writer_SendDataRaw(&Serial, sizeof(uint64));
#endif
}

////////////////////////////////////////////////////////////////////////////////
void Writer_TailOnConnect(uint32 MaxBytes /*= 0*/)
{
	FProfilerScope _(__func__);
	
	// If there's no tail being maintained then there is nothing to send.
	if (!GPacketRing.IsActive())
	{
		return;
	}

	GPacketRing.IterateRanges([] (const FPacketRing::FRange& Range)
	{
		Writer_SendDataRaw(Range.Data, Range.Size);
	}, MaxBytes);
}

////////////////////////////////////////////////////////////////////////////////
void Writer_InitializeTail(int32 BufferSize)
{
#if defined(STRESS_PACKET_RING)
	static void	StressRingPacket();
	StressRingPacket();
#endif

	if (BufferSize <= 0)
	{
		return;
	}

	// Round up to 1K and clamp the size. There has to be a sensible amount of
	// buffer size for tail tracing to work or be useful.
	uint32 Rounding = (1 << 10) - 1;
	BufferSize = (BufferSize + Rounding) & ~Rounding;
	if (BufferSize < (128 << 10))
	{
		BufferSize = 128 << 10;
	}

	GPacketRing.Initialize(BufferSize);
}

////////////////////////////////////////////////////////////////////////////////
bool Writer_IsTailing()
{
	return GPacketRing.IsActive();
}

////////////////////////////////////////////////////////////////////////////////
void Writer_ShutdownTail()
{
	GPacketRing.Shutdown();
}



////////////////////////////////////////////////////////////////////////////////
#if defined(STRESS_PACKET_RING)
static void StressRingPacket()
{
	FPacketRing Ring;
	Ring.Initialize(300);

	uint32 Bits = 0x0493'0493;
	for (int32 i = 0; i < 1024; ++i)
	{
		FTidPacket* Packet = Ring.Append<FTidPacket>((Bits & 0x1f) + 6);
		Packet->ThreadId = i;

		Ring.IterateRanges([] (const FPacketRing::FRange&)
		{
			/* nop */
		});

		Bits = (Bits ^ 0xa93a'93a9) * 0x0493;
	}

	for (int32 i = 7; i < 448; i += 67)
	{
		if (auto* Packet = Ring.Append<FTidPacket>(i))
		{
			Packet->ThreadId = 0;
		}
	}
}
#endif // STRESS_PACKET_RING

} // namespace Private
} // namespace Trace
} // namespace UE

#endif // TRACE_PRIVATE_MINIMAL_ENABLED

/*

FPacketRing ring-buffers packets. Internally the buffer is divided up into two
ranges; [0-Cursor) and [Left-Right) which are initially empty;

 0                                                                      L
 C----------------------------------------------------------------------R

A packet consists of a size and a opaque blob of data. Reading the sizes allows
one to stride through the packets.
                                                                        L
 0[SZ]==============>[SZ]=============>[SZ]=======>C--------------------R

Eventually the next packet will not fit in the buffer because the next cursor (N)
is off the buffer's end;
                                                                        L
 0[SZ]==============>[SZ]=============>[SZ]=======>[SZ]==========>C-----R
                                                                  [SZ]========>N

When this happens the 0-Cursor range is transferred to Left-Right and the 0-Cursor
range is set such that it can contain the new packet being added.

 L[SZ]==============>[SZ]=============>[SZ]=======>[SZ]==========>R-----|
 0[SZ]========>C

The two ranges now overlap so packets are then removed from Left until there is
enough space for the new packet.

 0[SZ]========>C-----L[SZ]============>[SZ]=======>[SZ]==========>R-----|

The Left-Right range has the oldest packets. Left will eventually advance to meet
Right at which point the Left-Right range becomes empty The process above repeats
as if the buffer was being filled for the first time.

*/
