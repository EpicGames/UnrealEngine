// Copyright Epic Games, Inc. All Rights Reserved.

#include "ThreadRegistry.h"

#if TRACE_PRIVATE_MINIMAL_ENABLED

#include "Trace/Detail/Atomic.h"
#include "Trace/Platform.h"

namespace UE::Trace::Private {

////////////////////////////////////////////////////////////////////////////////
void*	Writer_MemoryAllocate(SIZE_T, uint32);
void	Writer_MemoryFree(void*, uint32);

extern FStatistics GTraceStatistics;

static volatile FThreadGroupNode*				GGroupsHead;			// = nullptr;
static volatile FThreadGroupNode*				GGroupsTail;			// = nullptr;
static volatile FThreadGroupNode*				GCurrentGroup;			// = nullptr;
static volatile FThreadInfoNode*				GUngroupedThreadsHead;	// = nullptr;
static volatile FThreadInfoNode*				GUngroupedThreadsTail;	// = nullptr;
static volatile FThreadRegistryBlock*			GRegistryBlockHead;		// = nullptr;
static volatile uint8							GRegistryAllocLock;		// = 0;

////////////////////////////////////////////////////////////////////////////////
FThreadRegistryBlock* RegistryAllocateBlock()
{
	FThreadRegistryBlock* Block = (FThreadRegistryBlock*)Writer_MemoryAllocate(BlockAllocSize, 16);
	Block->Size = BlockAllocSize - sizeof(FThreadRegistryBlock);
	Block->Allocated = 0;
	Block->Next = nullptr;
#if TRACE_PRIVATE_STATISTICS
	AtomicAddRelaxed(&GTraceStatistics.ThreadRegistryAllocated, uint32(BlockAllocSize));
#endif
	return Block;
}

////////////////////////////////////////////////////////////////////////////////
template <typename T>
static T* RegistryAllocate()
{
	constexpr uint32 Size = sizeof(T);
	static_assert(Size % 8 == 0);

	FThreadRegistryBlock* Block;
	while (true)
	{
		Block = (FThreadRegistryBlock*)AtomicLoadAcquire(&GRegistryBlockHead);
		if (Block)
		{
			// Claim some space in current block
			uint32 Allocated = AtomicLoadRelaxed(&Block->Allocated);
			const uint32 Free = Block->Size - Allocated;
			if (Free >= Size)
			{
				uint32 NewAllocated = Allocated + Size;
				if (AtomicCompareExchangeRelease(&Block->Allocated, NewAllocated, Allocated))
				{
					return (T*) &Block->Data[Allocated];
				}
				continue;
			}
		}

		// Only one thread is allowed to allocate at one time
		if (!AtomicCompareExchangeAcquire(&GRegistryAllocLock, uint8(1u), uint8(0u)))
		{
			ThreadSleep(0);
			continue;
		}

		// If the block is still the same, we are safe to allocate
		if (AtomicLoadAcquire(&GRegistryBlockHead) == Block)
		{
			volatile FThreadRegistryBlock* NewBlock = RegistryAllocateBlock();
			NewBlock->Next = AtomicLoadRelaxed(&GRegistryBlockHead);
			AtomicStoreRelease(&GRegistryBlockHead, NewBlock);
		}
		AtomicStoreRelease(&GRegistryAllocLock, uint8(0u));
	}
}

////////////////////////////////////////////////////////////////////////////////
static FThreadGroupNode* CreateThreadGroup(const ANSICHAR* Name)
{
	FThreadGroupNode* Group = RegistryAllocate<FThreadGroupNode>();
	Group->Next = nullptr;
	Group->ThreadInfoHead = nullptr;

	if (Name != nullptr)
	{
		memcpy(Group->GroupName, Name, strlen(Name) + 1);
	}
	else
	{
		Group->GroupName[0] = '\0';
	}

	return Group;
}

////////////////////////////////////////////////////////////////////////////////
static FThreadInfoNode* CreateThreadInfo(const ANSICHAR* Name, uint32 ThreadId, uint32 SystemId, int32 SortHint)
{
	FThreadInfoNode* Thread = RegistryAllocate<FThreadInfoNode>();
	Thread->Next = nullptr;
	Thread->ThreadId = ThreadId;
	Thread->SystemId = SystemId;
	Thread->SortHint = SortHint;
	memcpy(Thread->Name, Name, strlen(Name) + 1);

	return Thread;
}

////////////////////////////////////////////////////////////////////////////////
bool Writer_AddThreadGroupBegin(const ANSICHAR* Name)
{
	volatile FThreadGroupNode* CurrentGroup = AtomicLoadAcquire(&GCurrentGroup);
	if (CurrentGroup != nullptr && !strcmp((const ANSICHAR*)CurrentGroup->GroupName, Name))
	{
		// Check to see if we're creating a dupe of the current ThreadGroup.
		// This can result in Kilobytes of repeating data in the registry.
		return false;
	}

	volatile FThreadGroupNode* NewGroup = CreateThreadGroup(Name);
	while (true)
	{
		volatile FThreadGroupNode* CurrentTail = AtomicLoadAcquire(&GGroupsTail);
		if (CurrentTail == nullptr)
		{
			// Race check to see if this is the first thread trying to set the initial group
			if (AtomicCompareExchangeRelease(&GGroupsHead, NewGroup, (volatile FThreadGroupNode*)nullptr))
			{
				AtomicStoreRelease(&GGroupsTail, NewGroup);
				break;
			}
		}
		else
		{
			if (AtomicCompareExchangeRelease(&CurrentTail->Next, NewGroup, (volatile FThreadGroupNode*)nullptr))
			{
				AtomicStoreRelease(&GGroupsTail, NewGroup);
				break;
			}
		}
	}

	// Last write will win if several threads call GroupBegin...
	AtomicStoreRelease(&GCurrentGroup, NewGroup);
	return true;
}

////////////////////////////////////////////////////////////////////////////////
void Writer_AddThreadGroupEnd()
{
	AtomicStoreRelease(&GCurrentGroup, (volatile FThreadGroupNode*)nullptr);
}

////////////////////////////////////////////////////////////////////////////////
const ANSICHAR* Writer_AddThreadInfo(uint32 ThreadId, uint32 SystemId, int32 SortHint, const ANSICHAR* Name)
{
	volatile FThreadInfoNode* NewThread = CreateThreadInfo(Name, ThreadId, SystemId, SortHint);

	volatile FThreadGroupNode* CurrentGroup = AtomicLoadAcquire(&GCurrentGroup);
	if (CurrentGroup == nullptr)
	{
		// CAS Loop to avoid any races
		while (true)
		{
			volatile FThreadInfoNode* CurrentTail = AtomicLoadAcquire(&GUngroupedThreadsTail);
			if (CurrentTail == nullptr)
			{
				// Same race applies for Ungrouped threads.
				if (AtomicCompareExchangeRelease(&GUngroupedThreadsHead, NewThread, (volatile FThreadInfoNode*)nullptr))
				{
					AtomicStoreRelease(&GUngroupedThreadsTail, NewThread);
					return nullptr;
				}
			}
			else
			{
				if (AtomicCompareExchangeRelease(&CurrentTail->Next, NewThread, (volatile FThreadInfoNode*)nullptr))
				{
					AtomicStoreRelease(&GUngroupedThreadsTail, NewThread);
					return nullptr;
				}
			}
		}
	}

	volatile FThreadInfoNode* CurrentThread;
	do
	{
		CurrentThread = AtomicLoadRelaxed(&CurrentGroup->ThreadInfoHead);
		AtomicStoreRelaxed(&NewThread->Next, CurrentThread);
	}
	while (!AtomicCompareExchangeRelease(&CurrentGroup->ThreadInfoHead, NewThread, CurrentThread));
	return (const ANSICHAR*)CurrentGroup->GroupName;
}

////////////////////////////////////////////////////////////////////////////////
void Writer_EnumerateThreadGroups(FThreadGroupCallback Callback)
{
	for (volatile FThreadGroupNode* Group = AtomicLoadAcquire(&GGroupsHead); Group != nullptr; Group = AtomicLoadRelaxed(&Group->Next))
	{
		Callback(Group);
	}
}

////////////////////////////////////////////////////////////////////////////////
void Writer_EnumerateUngroupedThreads(FThreadInfoCallback Callback)
{
	for (volatile FThreadInfoNode* UngroupedThread = AtomicLoadAcquire(&GUngroupedThreadsHead); UngroupedThread != nullptr; UngroupedThread = AtomicLoadRelaxed(&UngroupedThread->Next))
	{
		Callback(UngroupedThread);
	}
}

////////////////////////////////////////////////////////////////////////////////
FThreadGroupNode* Writer_GetCurrentGroup()
{
	return (FThreadGroupNode*)AtomicLoadAcquire(&GCurrentGroup);
}

} // namespace UE::Trace::Private

#endif // TRACE_PRIVATE_MINIMAL_ENABLED
