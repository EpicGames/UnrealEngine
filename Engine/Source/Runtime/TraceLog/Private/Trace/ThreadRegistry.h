// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Trace/Config.h"

#if TRACE_PRIVATE_MINIMAL_ENABLED

namespace UE::Trace::Private
{

////////////////////////////////////////////////////////////////////////////////
static constexpr uint8	MaxNameLength = 96;
static constexpr SIZE_T BlockAllocSize = 4 << 10;

////////////////////////////////////////////////////////////////////////////////
struct FThreadInfoNode
{
	volatile FThreadInfoNode*	Next;
	uint32						ThreadId;
	uint32						SystemId;
	int32						SortHint;
	uint32						_Padding; // ensures Name is 8-bytes aligned
	ANSICHAR					Name[MaxNameLength];
};

////////////////////////////////////////////////////////////////////////////////
struct FThreadGroupNode
{
	volatile FThreadGroupNode*	Next;
	volatile FThreadInfoNode*	ThreadInfoHead;
	ANSICHAR					GroupName[MaxNameLength];
};

////////////////////////////////////////////////////////////////////////////////
struct FThreadRegistryBlock
{
	uint32							Size;
	volatile uint32					Allocated;
	volatile FThreadRegistryBlock*	Next;
	uint8							Data[];
};

////////////////////////////////////////////////////////////////////////////////
typedef void (*FThreadGroupCallback)(volatile FThreadGroupNode*);
typedef void (*FThreadInfoCallback)(volatile FThreadInfoNode*);

////////////////////////////////////////////////////////////////////////////////
/**
 * Closes the current thread group and appends a new group to a linked list of groups.
 *
 * @param Name ANSI string name of the group (max 95 chars + null terminator).
 */
bool Writer_AddThreadGroupBegin(const ANSICHAR* Name);
/**
 * Closes the current thread group.
 */
void Writer_AddThreadGroupEnd();
/**
 * Adds thread information to the current thread group.
 *
 * @param ThreadId Trace thread identifier.
 * @param SystemId Operating system thread ID.
 * @param SortHint Hint for sorting threads in analysis (-1 for default).
 * @param Name ANSI string name of the thread (max 95 chars + null terminator).
 */
const ANSICHAR* Writer_AddThreadInfo(uint32 ThreadId, uint32 SystemId, int32 SortHint, const ANSICHAR* Name);
/**
 * Enumerates over the linked list of thread groups.
 *
 * @param Callback Pointer to the thread group node.
 */
void Writer_EnumerateThreadGroups(FThreadGroupCallback Callback);
/**
 * Enumerates over the linked list of ungrouped threads.
 *
 * @param Callback Pointer to the thread node.
 */
void Writer_EnumerateUngroupedThreads(FThreadInfoCallback Callback);
/**
 * @returns the currently focused thread group
 */
FThreadGroupNode* Writer_GetCurrentGroup();


} // namespace UE::Trace::Private

#endif // TRACE_PRIVATE_MINIMAL_ENABLED
