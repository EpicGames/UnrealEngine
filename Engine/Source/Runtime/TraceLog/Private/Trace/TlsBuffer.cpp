// Copyright Epic Games, Inc. All Rights Reserved.

#include "Trace/Config.h"

#if TRACE_PRIVATE_MINIMAL_ENABLED

#include "HAL/Platform.h" // for PLATFORM_BREAK
#include "Message.h"
#include "Platform.h"
#include "Trace/Detail/Atomic.h"
#include "Trace/Detail/Writer.inl"
#include "Trace/Detail/Transport.h"
#include "Trace/Trace.inl"
#include "WriteBufferRedirect.h"

namespace UE {
namespace Trace {
namespace Private {

////////////////////////////////////////////////////////////////////////////////
void				Writer_TailAppend(uint32, uint8* __restrict, uint32);
FWriteBuffer*		Writer_AllocateBlockFromPool();
uint32				Writer_GetThreadId();
void				Writer_FreeBlockListToPool(FWriteBuffer*, FWriteBuffer*);
void				Writer_MemoryFree(void* Address, uint32 Size);
void*				Writer_MemoryAllocate(SIZE_T Size, uint32 Alignment);
void				Writer_SendData(uint32, uint8 *__restrict, uint32);
FWriteBuffer*		Writer_NextBuffer(FWriteBuffer* CurrentBuffer);
extern uint64		GStartCycle;
extern FStatistics	GTraceStatistics;

////////////////////////////////////////////////////////////////////////////////
UE_TRACE_MINIMAL_EVENT_BEGIN($Trace, ThreadTiming, NoSync)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, BaseTimestamp)
UE_TRACE_MINIMAL_EVENT_END()

////////////////////////////////////////////////////////////////////////////////
struct FThreadBuffer
{
	FWriteBuffer* FirstBuffer;
	uint32 DrainBufferLimit;
	uint32 Padding;
};


////////////////////////////////////////////////////////////////////////////////
#define T_ALIGN alignas(PLATFORM_CACHE_LINE_SIZE)
static FWriteBuffer						GNullWriteBuffer	= { {}, static_cast<uint16>(~0u), 0, 0, 0, 0, nullptr, nullptr, nullptr, nullptr, (uint8*)&GNullWriteBuffer };
thread_local FTlsState					GTlsState = {&GNullWriteBuffer, nullptr};
static FWriteBuffer* __restrict			GActiveThreadList;	// = nullptr;
T_ALIGN static FWriteBuffer* volatile	GNewThreadList;		// = nullptr;
static FThreadBuffer*					GThreadBufferArray;	// = nullptr;
static uint32							GThreadBufferArrayCapacity;	// = 0;
#undef T_ALIGN

////////////////////////////////////////////////////////////////////////////////
#if !IS_MONOLITHIC
TRACELOG_API FWriteBuffer* Writer_GetBuffer()
{
	// Thread locals and DLLs don't mix so for modular builds we are forced to
	// export this function to access thread-local variables.
	return GTlsState.WriteBuffer;
}
#endif

////////////////////////////////////////////////////////////////////////////////
TRACELOG_API bool Writer_OnConnectActivate(FTlsState& OutPrevState)
{
	FWriteBuffer* OnConnectBuffer = FOnConnectRedirectScope::GetBuffer();
	if (OnConnectBuffer == nullptr)
	{
		return false;
	}
	OutPrevState = GTlsState;
	GTlsState.WriteBuffer = OnConnectBuffer;
	GTlsState.NextBufferFn = &FOnConnectRedirectScope::NextBuffer;
	return true;
}

////////////////////////////////////////////////////////////////////////////////
TRACELOG_API void Writer_OnConnectDeactivate(const FTlsState& PrevState)
{
	GTlsState = PrevState;
}

////////////////////////////////////////////////////////////////////////////////
static FWriteBuffer* Writer_NextBufferInternal(FWriteBuffer* CurrentBuffer)
{
	// TraceData is used to catch traced events that occur when allocating
	// memory. Such events should be tightly controlled and kept to a minimum.
	// There is not a lot of buffer space available in TraceData to avoid
	// exhausting available stack and leave usable space in the new buffer.
	TBoundedRedirectScope<2048> TraceData;

#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST
	if (CurrentBuffer->ThreadId == decltype(TraceData)::ActiveRedirection)
	{
		// If we've reached this point we're are in an unrecoverable situation.
		// Allocating memory results in so many trace events that there is
		// insufficient space to store them. We can't allocate more space, because
		// that would result in more traced events, and so on...
		UE_TRACE_MESSAGE(WriterError, "Redirect buffer full, trace will be corrupt!");
		PLATFORM_BREAK();
	}
#endif

	FWriteBuffer* NextBuffer = Writer_AllocateBlockFromPool();
	NextBuffer->SetCursor((uint8*)NextBuffer - NextBuffer->Size);
	NextBuffer->Reaped = NextBuffer->ReadCursor();
	NextBuffer->EtxOffset = 0 - int32(sizeof(FWriteBuffer));
	AtomicStoreRelaxed(&(NextBuffer->NextBuffer), (FWriteBuffer*)nullptr);

	// Add any capture events into the buffer.
	if (uint32 RedirectSize = TraceData.GetSize())
	{
		memcpy(NextBuffer->ReadCursor(), TraceData.GetData(), RedirectSize);
		NextBuffer->MoveCursor((uint16)RedirectSize);
	}

	TraceData.Abandon();

	GTlsState.WriteBuffer = NextBuffer;

	NextBuffer->Committed = NextBuffer->ReadCursor();

	if (CurrentBuffer == &GNullWriteBuffer)
	{
		NextBuffer->ThreadId = uint16(Writer_GetThreadId());
		NextBuffer->PrevTimestamp = TimeGetTimestamp();

		UE_TRACE_MINIMAL_LOG($Trace, ThreadTiming, TraceLogChannel)
			<< ThreadTiming.BaseTimestamp(NextBuffer->PrevTimestamp - GStartCycle);

		// Add this next buffer to the active list.
		for (;; PlatformYield())
		{
			NextBuffer->NextThread = AtomicLoadRelaxed(&GNewThreadList);
			if (AtomicCompareExchangeRelease(&GNewThreadList, NextBuffer, NextBuffer->NextThread))
			{
				break;
			}
		}
	}
	else
	{
	#if TRACE_PRIVATE_STATISTICS
		// Note that we only track the emitted bytes when a buffer is filled as the overhead
		// of doing on every event would be too large.
		const uint64 Used = CurrentBuffer->ReadCursor() - ((uint8*)CurrentBuffer - CurrentBuffer->Size);
		AtomicAddRelaxed(&GTraceStatistics.BytesEmitted, Used);
	#endif

		NextBuffer->ThreadId = CurrentBuffer->ThreadId;
		NextBuffer->PrevTimestamp = CurrentBuffer->PrevTimestamp;
		AtomicStoreRelease(&(CurrentBuffer->NextBuffer), NextBuffer);

		// Retire current buffer.
		int32 EtxOffset = int32(PTRINT((uint8*)(CurrentBuffer) - CurrentBuffer->ReadCursor()));
		AtomicStoreRelease(&(CurrentBuffer->EtxOffset), EtxOffset);
	}

	return NextBuffer;
}

////////////////////////////////////////////////////////////////////////////////
TRACELOG_API FWriteBuffer* Writer_NextBuffer(FWriteBuffer* CurrentBuffer)
{
	// Allow us to redirect next buffer function (e.g. growable redirects). If
	// not set fall back to the default implementation.
	if (GTlsState.NextBufferFn)
	{
		return GTlsState.NextBufferFn(CurrentBuffer);
	}
	return Writer_NextBufferInternal(CurrentBuffer);
}

////////////////////////////////////////////////////////////////////////////////
static void Writer_ReallocateThreadBufferArray(uint32 NewArrayCapacity)
{
	FProfilerScope _(__func__);

	uint32 NewSize = NewArrayCapacity * sizeof(FThreadBuffer);
	FThreadBuffer* NewArray = static_cast<FThreadBuffer*>(Writer_MemoryAllocate(NewSize, 16));
#if TRACE_PRIVATE_STATISTICS
	AtomicAddRelaxed(&GTraceStatistics.FixedBufferAllocated, NewSize);
#endif

	if (GThreadBufferArrayCapacity > 0)
	{
		const uint32 OldSize = GThreadBufferArrayCapacity * sizeof(FThreadBuffer);
		memcpy(NewArray, GThreadBufferArray, OldSize);
		Writer_MemoryFree(GThreadBufferArray, OldSize);
#if TRACE_PRIVATE_STATISTICS
		AtomicSubRelaxed(&GTraceStatistics.FixedBufferAllocated, OldSize);
#endif
	}

	GThreadBufferArray = NewArray;
	GThreadBufferArrayCapacity = NewArrayCapacity;
}

////////////////////////////////////////////////////////////////////////////////
static bool Writer_DrainBuffer(uint32 ThreadId, FWriteBuffer* Buffer)
{
	uint8* Committed = AtomicLoadAcquire((uint8**)&Buffer->Committed);

	// Send as much as we can.
	if (uint32 SizeToReap = uint32(Committed - Buffer->Reaped))
	{
#if TRACE_PRIVATE_STATISTICS
		FPlatformAtomics::AtomicStore_Relaxed((int64*)&GTraceStatistics.BytesTraced, FPlatformAtomics::AtomicRead_Relaxed((int64*)&GTraceStatistics.BytesTraced) + SizeToReap);
#endif

		Writer_TailAppend(ThreadId, Buffer->Reaped, SizeToReap);
		Buffer->Reaped = Committed;
	}

	// Is this buffer still in use?
	int32 EtxOffset = AtomicLoadAcquire(&Buffer->EtxOffset);
	return ((uint8*)Buffer - EtxOffset) > Committed;
}

////////////////////////////////////////////////////////////////////////////////
void Writer_DrainBuffers()
{
	FProfilerScope _(__func__);

	struct FRetireList
	{
		FWriteBuffer* __restrict Head = nullptr;
		FWriteBuffer* __restrict Tail = nullptr;

		void Insert(FWriteBuffer* __restrict Buffer)
		{
			AtomicStoreRelaxed(&(Buffer->NextBuffer), Head);
			Head = Buffer;
			Tail = (Tail != nullptr) ? Tail : Head;
		}
	};

	// Claim ownership of any new thread buffer lists
	FWriteBuffer* __restrict NewThreadList = AtomicExchangeAcquire(&GNewThreadList, (FWriteBuffer*)nullptr);

	// Reverse the new threads list so they're more closely ordered by age
	// when sent out.
	FWriteBuffer* __restrict NewThreadCursor = NewThreadList;
	NewThreadList = nullptr;
	while (NewThreadCursor != nullptr)
	{
		FWriteBuffer* __restrict NextThread = NewThreadCursor->NextThread;

		NewThreadCursor->NextThread = NewThreadList;
		NewThreadList = NewThreadCursor;

		NewThreadCursor = NextThread;
	}

	FRetireList RetireList;
	uint32 RetiredCount = 0;

	FWriteBuffer* __restrict ActiveThreadList = GActiveThreadList;
	GActiveThreadList = nullptr;

	// Now we've two lists of known and new threads. Each of these lists in turn is
	// a list of that thread's buffers (where it is writing trace events to).

	// Step one: Identify the buffers available now, for each available thread.
	uint32 ThreadCount = 0;
	for (FWriteBuffer* __restrict Buffer : { ActiveThreadList, NewThreadList })
	{
		// For each thread...
		for (FWriteBuffer* __restrict NextThread; Buffer != nullptr; Buffer = NextThread)
		{
			NextThread = Buffer->NextThread;

			if (ThreadCount >= GThreadBufferArrayCapacity)
			{
				uint32 NewArrayCapacity = (GThreadBufferArrayCapacity == 0) ? 256 : GThreadBufferArrayCapacity << 1;
				Writer_ReallocateThreadBufferArray(NewArrayCapacity);
			}

			// Count how many buffers are available now, for the current thread.
			uint32 NumBuffers = 1;
			for (FWriteBuffer* __restrict CrtBuffer = Buffer; CrtBuffer != nullptr; ++NumBuffers)
			{
				CrtBuffer = AtomicLoadAcquire(&(CrtBuffer->NextBuffer));
			}
			GThreadBufferArray[ThreadCount].FirstBuffer = Buffer;
			GThreadBufferArray[ThreadCount].DrainBufferLimit = NumBuffers;
			++ThreadCount;
		}
	}

	// Step two: Drain only the threads and their buffers identified in step one.
	for (uint32 ThreadIndex = 0; ThreadIndex < ThreadCount; ++ThreadIndex)
	{
		FWriteBuffer* __restrict Buffer = GThreadBufferArray[ThreadIndex].FirstBuffer;
		uint32 ThreadId = Buffer->ThreadId;
		uint32 DrainBufferLimit = GThreadBufferArray[ThreadIndex].DrainBufferLimit;

		// For each of the thread's buffers...
		for (FWriteBuffer* __restrict NextBuffer; Buffer != nullptr; Buffer = NextBuffer)
		{
			if (--DrainBufferLimit == 0)
			{
				// Ignore buffers added after the "step one" above.
				break;
			}

			if (Writer_DrainBuffer(ThreadId, Buffer))
			{
				break;
			}

			// Retire the buffer.
			NextBuffer = AtomicLoadAcquire(&(Buffer->NextBuffer));
			RetireList.Insert(Buffer);
			++RetiredCount;
		}

		if (Buffer != nullptr)
		{
			Buffer->NextThread = GActiveThreadList;
			GActiveThreadList = Buffer;
		}
	}

	// Put the retirees we found back into the system again.
	if (RetireList.Head != nullptr)
	{
		Writer_FreeBlockListToPool(RetireList.Head, RetireList.Tail);
		AtomicAddRelaxed(&GTraceStatistics.BlockPoolFreeBlocks, RetiredCount);
	}
}

////////////////////////////////////////////////////////////////////////////////
void Writer_DrainLocalBuffers()
{
	if (GTlsState.WriteBuffer == &GNullWriteBuffer)
	{
		return;
	}

	const uint32 LocalThreadId = GTlsState.WriteBuffer->ThreadId;

	struct FRetireList
	{
		FWriteBuffer* __restrict Head = nullptr;
		FWriteBuffer* __restrict Tail = nullptr;

		void Insert(FWriteBuffer* __restrict Buffer)
		{
			AtomicStoreRelaxed(&(Buffer->NextBuffer), Head);
			Head = Buffer;
			Tail = (Tail != nullptr) ? Tail : Head;
		}
	};

	FRetireList RetireList;
	uint32 RetiredCount = 0;

	FWriteBuffer* __restrict ActiveThreadList = GActiveThreadList;
	GActiveThreadList = nullptr;

	{
		FWriteBuffer* __restrict Buffer = ActiveThreadList;
		FWriteBuffer* __restrict NextThread;

		// For each thread...
		for (; Buffer != nullptr; Buffer = NextThread)
		{
			NextThread = Buffer->NextThread;
			uint32 ThreadId = Buffer->ThreadId;

			if (ThreadId == LocalThreadId)
			{
				// For each of the thread's buffers...
				for (FWriteBuffer* __restrict NextBuffer; Buffer != nullptr; Buffer = NextBuffer)
				{
					if (Writer_DrainBuffer(ThreadId, Buffer))
					{
						break;
					}

					// Retire the buffer
					NextBuffer = AtomicLoadAcquire(&(Buffer->NextBuffer));
					RetireList.Insert(Buffer);
					++RetiredCount;
				}
			}

			if (Buffer != nullptr)
			{
				Buffer->NextThread = GActiveThreadList;
				GActiveThreadList = Buffer;
			}
		}
	}

	// Put the retirees we found back into the system again.
	if (RetireList.Head != nullptr)
	{
		Writer_FreeBlockListToPool(RetireList.Head, RetireList.Tail);
		AtomicAddRelaxed(&GTraceStatistics.BlockPoolFreeBlocks, RetiredCount);
	}
}

////////////////////////////////////////////////////////////////////////////////
void Writer_DrainBufferList(uint32 ThreadId, FWriteBuffer* List)
{
	FWriteBuffer* Buffer = List;
	do {
		uint8 *Committed = AtomicLoadAcquire((uint8 **)&Buffer->Committed);

		// Send as much as we can.
		if (uint32 SizeToReap = uint32(Committed - Buffer->Reaped))
		{
#if TRACE_PRIVATE_STATISTICS
			AtomicAddRelaxed(&GTraceStatistics.BytesTraced, uint64(SizeToReap));
#endif

			Writer_SendData(ThreadId, Buffer->Reaped, SizeToReap);
			Buffer->Reaped = Committed;
		}
	} while ((Buffer = Buffer->NextBuffer));
}

////////////////////////////////////////////////////////////////////////////////
void Writer_EndThreadBuffer()
{
	if (GTlsState.WriteBuffer == &GNullWriteBuffer)
	{
		return;
	}

	const PTRINT CurrentCursor = (PTRINT)GTlsState.WriteBuffer->ReadCursor();
	int32 EtxOffset = int32(PTRINT((uint8*)GTlsState.WriteBuffer - CurrentCursor));
	AtomicStoreRelease(&(GTlsState.WriteBuffer->EtxOffset), EtxOffset);
}

} // namespace Private
} // namespace Trace
} // namespace UE

#endif // TRACE_PRIVATE_MINIMAL_ENABLED
