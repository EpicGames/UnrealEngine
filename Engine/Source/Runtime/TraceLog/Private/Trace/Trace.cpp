// Copyright Epic Games, Inc. All Rights Reserved.

#include "Trace/Trace.inl" // should be Config.h :(
#include "Trace/Detail/Atomic.h"

#if TRACE_PRIVATE_MINIMAL_ENABLED

#include "Platform.h"
#include "Misc/CString.h"
#include "ThreadRegistry.h"
#include "Trace/Detail/Channel.h"


namespace UE {
namespace Trace {

////////////////////////////////////////////////////////////////////////////////
UE_TRACE_MINIMAL_CHANNEL_EXTERN(TraceLogChannel)

UE_TRACE_MINIMAL_EVENT_BEGIN($Trace, ThreadInfo, MaybeImportant|NoSync)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint32, ThreadId)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint32, SystemId)
	UE_TRACE_MINIMAL_EVENT_FIELD(int32, SortHint)
	UE_TRACE_MINIMAL_EVENT_FIELD(AnsiString, Name)
UE_TRACE_MINIMAL_EVENT_END()

UE_TRACE_MINIMAL_EVENT_BEGIN($Trace, ThreadGroupBegin, MaybeImportant|NoSync)
	UE_TRACE_MINIMAL_EVENT_FIELD(AnsiString, Name)
UE_TRACE_MINIMAL_EVENT_END()

UE_TRACE_MINIMAL_EVENT_BEGIN($Trace, ThreadGroupEnd, MaybeImportant|NoSync)
UE_TRACE_MINIMAL_EVENT_END()

////////////////////////////////////////////////////////////////////////////////
struct FThreadEventsRegistrator
{
	FThreadEventsRegistrator()
	{
		F$TraceThreadInfoFields::GetUid();
		F$TraceThreadGroupBeginFields::GetUid();
		F$TraceThreadGroupEndFields::GetUid();
	}
};

// Static registration to provoke event descriptions as these events can be first emitted after events have been described but sent in the same update loop.
static FThreadEventsRegistrator GThreadEventsRegistrator;

namespace Private
{

////////////////////////////////////////////////////////////////////////////////
void	Message_SetCallback(OnMessageFunc* Callback);
void	Writer_SetUpdateCallback(OnUpdateFunc* Callback);
void	Writer_MemorySetHooks(AllocFunc, FreeFunc);
void	Writer_Initialize(const FInitializeDesc&);
void	Writer_WorkerCreate();
void	Writer_UnsetBlockPoolLimit();
void	Writer_Shutdown();
void	Writer_Update();
bool	Writer_SendTo(const ANSICHAR*, uint32, uint32);
bool	Writer_WriteTo(const ANSICHAR*, uint32);
bool	Writer_RelayTo(UPTRINT, IoWriteFunc, IoCloseFunc, uint16);
bool	Writer_WriteSnapshotTo(const ANSICHAR*, uint32);
bool	Writer_SendSnapshotTo(const ANSICHAR*, uint32, uint32);
bool	Writer_IsTracing();
bool	Writer_IsTracingTo(uint32 (&OutSessionGuid)[4], uint32 (&OutTraceGuid)[4]);
bool	Writer_IsTailing();
bool	Writer_Stop();
uint32	Writer_GetThreadId();
void	Writer_Panic();

extern FStatistics GTraceStatistics;

UE_TRACE_MINIMAL_EVENT_BEGIN($Trace, TraceStall)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, StartCycle)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, EndCycle)
UE_TRACE_MINIMAL_EVENT_END()

void LogStall(uint64 Start, uint64 End)
{
	UE_TRACE_MINIMAL_LOG($Trace, TraceStall, true)
		<< TraceStall.StartCycle(Start)
		<< TraceStall.EndCycle(End);
}

void Writer_LogThreadInfo(uint32 ThreadId, uint32 SystemId, int32 SortHint, const ANSICHAR* Name, uint32 NameLen)
{
	UE_TRACE_MINIMAL_LOG($Trace, ThreadInfo, TraceLogChannel)
		<< ThreadInfo.ThreadId(ThreadId)
		<< ThreadInfo.SystemId(SystemId)
		<< ThreadInfo.SortHint(SortHint)
		<< ThreadInfo.Name(Name, NameLen);
}

void Writer_LogThreadGroupBegin(const ANSICHAR* GroupName, uint32 NameLen)
{
	UE_TRACE_MINIMAL_LOG($Trace, ThreadGroupBegin, TraceLogChannel)
		<< ThreadGroupBegin.Name(GroupName, NameLen);
}

void Writer_LogThreadGroupEnd()
{
	UE_TRACE_MINIMAL_LOG($Trace, ThreadGroupEnd, TraceLogChannel);
}

} // namespace Private



////////////////////////////////////////////////////////////////////////////////
template <int DestSize, typename SRC_TYPE>
static uint32 ToAnsiCheap(ANSICHAR (&Dest)[DestSize], const SRC_TYPE* Src)
{
	const SRC_TYPE* Cursor = Src;
	for (ANSICHAR& Out : Dest)
	{
		Out = ANSICHAR(*Cursor++ & 0x7f);
		if (Out == '\0')
		{
			break;
		}
	}
	Dest[DestSize - 1] = '\0';
	return uint32(UPTRINT(Cursor - Src));
};

////////////////////////////////////////////////////////////////////////////////
void SetMemoryHooks(AllocFunc Alloc, FreeFunc Free)
{
	Private::Writer_MemorySetHooks(Alloc, Free);
}

////////////////////////////////////////////////////////////////////////////////
void SetMessageCallback(OnMessageFunc* MessageFunc)
{
	Private::Message_SetCallback(MessageFunc);
}

////////////////////////////////////////////////////////////////////////////////
void SetUpdateCallback(OnUpdateFunc* UpdateFunc)
{
	Private::Writer_SetUpdateCallback(UpdateFunc);
}

////////////////////////////////////////////////////////////////////////////////
void Initialize(const FInitializeDesc& Desc)
{
	Private::Writer_Initialize(Desc);
	FChannel::Initialize();
}

////////////////////////////////////////////////////////////////////////////////
void Exit()
{
	Private::Writer_UnsetBlockPoolLimit();
}

////////////////////////////////////////////////////////////////////////////////
void Shutdown()
{
	Private::Writer_Shutdown();
}

////////////////////////////////////////////////////////////////////////////////
void Panic()
{
	static bool bHasPanicked = []() { 
		FChannel::PanicDisableAll();
		Private::Writer_Panic();
		return true;
	}();
}

////////////////////////////////////////////////////////////////////////////////
void Update()
{
	Private::Writer_Update();
}

////////////////////////////////////////////////////////////////////////////////
void GetStatistics(FStatistics& Out)
{
	Out.BytesSent = Private::AtomicLoadRelaxed(&Private::GTraceStatistics.BytesSent);
	Out.BytesTraced = Private::AtomicLoadRelaxed(&Private::GTraceStatistics.BytesTraced);
	Out.BytesEmitted = Private::AtomicLoadRelaxed(&Private::GTraceStatistics.BytesEmitted);
	Out.MemoryUsed = Private::AtomicLoadRelaxed(&Private::GTraceStatistics.MemoryUsed);
	Out.BlockPoolAllocated = Private::AtomicLoadRelaxed(&Private::GTraceStatistics.BlockPoolAllocated);
	Out.BlockPoolFreeBlocks = Private::AtomicLoadRelaxed(&Private::GTraceStatistics.BlockPoolFreeBlocks);
	Out.BlockPoolAllocatedBlocks = Private::AtomicLoadRelaxed(&Private::GTraceStatistics.BlockPoolAllocatedBlocks);
	Out.SharedBufferAllocated = Private::AtomicLoadRelaxed(&Private::GTraceStatistics.SharedBufferAllocated);
	Out.FixedBufferAllocated = Private::AtomicLoadRelaxed(&Private::GTraceStatistics.FixedBufferAllocated);
	Out.CacheAllocated = Private::AtomicLoadRelaxed(&Private::GTraceStatistics.CacheAllocated);
	Out.CacheUsed = Private::AtomicLoadRelaxed(&Private::GTraceStatistics.CacheUsed);
	Out.CacheWaste = Private::AtomicLoadRelaxed(&Private::GTraceStatistics.CacheWaste);
	Out.ThreadRegistryAllocated = Private::AtomicLoadRelaxed(&Private::GTraceStatistics.ThreadRegistryAllocated);
}

////////////////////////////////////////////////////////////////////////////////
bool SendTo(const TCHAR* InHost, uint32 Port, uint16 Flags)
{
	char Host[256];
	ToAnsiCheap(Host, InHost);
	return Private::Writer_SendTo(Host, Flags, Port);
}

////////////////////////////////////////////////////////////////////////////////
bool WriteTo(const TCHAR* InPath, uint16 Flags)
{
	char Path[512];
	ToAnsiCheap(Path, InPath);
	return Private::Writer_WriteTo(Path, Flags);
}

////////////////////////////////////////////////////////////////////////////////
bool RelayTo(UPTRINT InHandle, IoWriteFunc WriteFunc, IoCloseFunc CloseFunc, uint16 Flags)
{
	return Private::Writer_RelayTo(InHandle, WriteFunc, CloseFunc, Flags);
}

////////////////////////////////////////////////////////////////////////////////
bool WriteSnapshotTo(const TCHAR* InPath, uint32 MaxTailSize)
{
	char Path[512];
	ToAnsiCheap(Path, InPath);
	return Private::Writer_WriteSnapshotTo(Path, MaxTailSize);
}

////////////////////////////////////////////////////////////////////////////////
bool SendSnapshotTo(const TCHAR* InHost, uint32 InPort, uint32 MaxTailSize)
{
	char Host[512];
	ToAnsiCheap(Host, InHost);
	return Private::Writer_SendSnapshotTo(Host, InPort, MaxTailSize);
}

////////////////////////////////////////////////////////////////////////////////
bool IsTracing()
{
	return Private::Writer_IsTracing();
}

////////////////////////////////////////////////////////////////////////////////
bool IsTracingTo(uint32 (&OutSessionGuid)[4], uint32 (&OutTraceGuid)[4])
{
	return Private::Writer_IsTracingTo(OutSessionGuid, OutTraceGuid);
}

////////////////////////////////////////////////////////////////////////////////
bool IsTailing()
{
	return Private::Writer_IsTailing();
}

////////////////////////////////////////////////////////////////////////////////
bool Stop()
{
	return Private::Writer_Stop();
}

////////////////////////////////////////////////////////////////////////////////
bool IsChannel(const TCHAR* ChannelName)
{
	ANSICHAR ChannelNameA[64];
	ToAnsiCheap(ChannelNameA, ChannelName);
	return FChannel::FindChannel(ChannelNameA) != nullptr;
}

////////////////////////////////////////////////////////////////////////////////
bool ToggleChannel(const TCHAR* ChannelName, bool bEnabled)
{
	ANSICHAR ChannelNameA[64];
	ToAnsiCheap(ChannelNameA, ChannelName);
	return FChannel::Toggle(ChannelNameA, bEnabled);
}

////////////////////////////////////////////////////////////////////////////////
void EnumerateChannels(ChannelIterFunc IterFunc, void* User)
{
	struct FCallbackDataWrapper
	{
		ChannelIterFunc* Func;
		void* User;
	};

	FCallbackDataWrapper Wrapper;
	Wrapper.Func = IterFunc;
	Wrapper.User = User;

	FChannel::EnumerateChannels([](const FChannelInfo& Info, void* User)
		{
			FCallbackDataWrapper* Wrapper = (FCallbackDataWrapper*)User;
			(*Wrapper).Func(Info.Name, Info.bIsEnabled, (*Wrapper).User);
			return true;
		}, &Wrapper);
}

////////////////////////////////////////////////////////////////////////////////
void EnumerateChannels(ChannelIterCallback IterFunc, void* User)
{
	FChannel::EnumerateChannels(IterFunc, User);
}

////////////////////////////////////////////////////////////////////////////////
void StartWorkerThread()
{
	Private::Writer_WorkerCreate();
}

////////////////////////////////////////////////////////////////////////////////
FChannel* FindChannel(const TCHAR* ChannelName)
{
	ANSICHAR ChannelNameA[64];
	ToAnsiCheap(ChannelNameA, ChannelName);
	return FChannel::FindChannel(ChannelNameA);
}

////////////////////////////////////////////////////////////////////////////////
FChannel* FindChannel(FChannelId ChannelId)
{
	return FChannel::FindChannel(ChannelId);
}

////////////////////////////////////////////////////////////////////////////////
void ThreadRegister(const TCHAR* Name, uint32 SystemId, int32 SortHint)
{
	using namespace Private;
	FProfilerScope _(__func__);

	ANSICHAR NameA[96];

	uint32 ThreadId = Writer_GetThreadId();
	uint32 NameLen = ToAnsiCheap(NameA, Name);

	const ANSICHAR* GroupName = Writer_AddThreadInfo(ThreadId, SystemId, SortHint, NameA);
	if (Writer_IsTracing())
	{
		if (GroupName)
		{
			// We need to always emit the group the thread belongs to as we cannot guarantee thread synchronization.
			Writer_LogThreadGroupBegin(GroupName, (uint32)strlen(GroupName) + 1);
			Writer_LogThreadInfo(ThreadId, SystemId, SortHint, NameA, NameLen);
			Writer_LogThreadGroupEnd();
		}
		else
		{
			Writer_LogThreadInfo(ThreadId, SystemId, SortHint, NameA, NameLen);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////
void ThreadGroupBegin(const TCHAR* Name)
{
	using namespace Private;
	FProfilerScope _(__func__);

	ANSICHAR NameA[96];
	ToAnsiCheap(NameA, Name);
	Writer_AddThreadGroupBegin(NameA);
}

////////////////////////////////////////////////////////////////////////////////
void ThreadGroupEnd()
{
	using namespace Private;
	FProfilerScope _(__func__);

	Writer_AddThreadGroupEnd();
}

} // namespace Trace
} // namespace UE

#else

// Workaround for module not having any exported symbols
TRACELOG_API int TraceLogExportedSymbol = 0;

#endif // TRACE_PRIVATE_MINIMAL_ENABLED
