// Copyright Epic Games, Inc. All Rights Reserved.
#include "WriteBufferRedirect.h"
#include "Trace/Config.h"

namespace UE {
namespace Trace {
namespace Private {

#if TRACE_PRIVATE_MINIMAL_ENABLED

////////////////////////////////////////////////////////////////////////////////
FWriteBuffer* FOnConnectRedirectScope::Tail = nullptr;

////////////////////////////////////////////////////////////////////////////////
FWriteBuffer* FOnConnectRedirectScope::NextBuffer(FWriteBuffer* CurrentBuffer)
{
	Tail = TWriteBufferQueue<OnConnectBufferSize>::NextBuffer(CurrentBuffer);
	return Tail;
}

////////////////////////////////////////////////////////////////////////////////

#endif //TRACE_PRIVATE_MINIMAL_ENABLED

} // namespace Private
} // namespace Trace
} // namespace UE
