// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Trace/Config.h"

#if TRACE_PRIVATE_MINIMAL_ENABLED

// HEADER_UNIT_SKIP - Internal
#include "Trace/Trace.h"
#include "Trace/Detail/Writer.inl"
#include "Trace/Detail/Transport.h"
#include <stdint.h>

namespace UE {
namespace Trace {
namespace Private {

////////////////////////////////////////////////////////////////////////////////
void* Writer_MemoryAllocate(SIZE_T, uint32);
void  Writer_MemoryFree(void*, uint32);

extern thread_local FTlsState GTlsState;
struct FWriteBuffer;

////////////////////////////////////////////////////////////////////////////////

/**
 * A buffer queue of traced data. Usable for redirecting behaviour.
 */
template <uint16 BufferSize>
class TWriteBufferQueue 
{
public:
	TWriteBufferQueue();
	~TWriteBufferQueue();
	static FWriteBuffer* AllocBuffer();
	static FWriteBuffer* FreeBuffer(FWriteBuffer*);
	static FWriteBuffer* NextBuffer(FWriteBuffer*);

	FWriteBuffer*	Head;
};

////////////////////////////////////////////////////////////////////////////////
template <uint16 BufferSize>
TWriteBufferQueue<BufferSize>::TWriteBufferQueue()
{
	Head = AllocBuffer();
}

////////////////////////////////////////////////////////////////////////////////
template <uint16 BufferSize>
TWriteBufferQueue<BufferSize>::~TWriteBufferQueue()
{
	FWriteBuffer* NextBuffer = Head;
	while (NextBuffer)
	{
		NextBuffer = FreeBuffer(NextBuffer);
	}
}

////////////////////////////////////////////////////////////////////////////////
template <uint16 BufferSize>
FWriteBuffer* TWriteBufferQueue<BufferSize>::AllocBuffer()
{
	constexpr uint16 UsableBufferSize = BufferSize - sizeof(FWriteBuffer);
	static_assert(PLATFORM_CACHE_LINE_SIZE % alignof(FWriteBuffer) == 0);
	static_assert(UsableBufferSize % alignof(FWriteBuffer) == 0);
	static_assert(UsableBufferSize <= UINT16_MAX);
	static_assert(BufferSize % PLATFORM_CACHE_LINE_SIZE == 0); // _aligned_alloc requirement

	uint8* AllocBase = (uint8*) Writer_MemoryAllocate(BufferSize, PLATFORM_CACHE_LINE_SIZE);

	FWriteBuffer* NextBuffer = (FWriteBuffer*)(AllocBase + UsableBufferSize);
	NextBuffer->Size = UsableBufferSize;
	NextBuffer->CursorMaskInv = 0;

	// Reserve some space for packet header
	uint8* DataStart = AllocBase + sizeof(FTidPacketBase);
	NextBuffer->SetCursor(DataStart);
	NextBuffer->Committed = DataStart;
	NextBuffer->Reaped = DataStart;

	NextBuffer->EtxOffset = 0 - int32(sizeof(FWriteBuffer));
	NextBuffer->ThreadId = 0;
	NextBuffer->NextBuffer = nullptr;

	return NextBuffer;
}

////////////////////////////////////////////////////////////////////////////////
template <uint16 BufferSize>
FWriteBuffer* TWriteBufferQueue<BufferSize>::FreeBuffer(FWriteBuffer* Buffer)
{
	FWriteBuffer* NextBuffer = Buffer->NextBuffer;
	const uint32 AllocSize = sizeof(FWriteBuffer) + Buffer->Size;
	uint8* AllocBase = ((uint8*)Buffer) - Buffer->Size;

	Writer_MemoryFree(AllocBase, AllocSize);

	return NextBuffer;
}

////////////////////////////////////////////////////////////////////////////////
template <uint16 BufferSize>
FWriteBuffer* TWriteBufferQueue<BufferSize>::NextBuffer(FWriteBuffer* __restrict PrevBuffer)
{
	FWriteBuffer* NextBuffer = AllocBuffer();
	PrevBuffer->NextBuffer = NextBuffer;
	GTlsState.WriteBuffer = NextBuffer;
	return NextBuffer;
}


////////////////////////////////////////////////////////////////////////////////

/**
 * Unbounded redirect buffer. Will allocate new buffers with templated size 
 * while active.
 */
template <uint16 BufferSize>
class TRedirectScope : public TWriteBufferQueue<BufferSize>
{
public:
	enum : uint16 { ActiveGrowingRedirection = 0xfffe };

	TRedirectScope();
	~TRedirectScope();

	FWriteBuffer* Finish();
	static FWriteBuffer* NextBuffer(FWriteBuffer* CurrentBuffer);

private:
	void ResetThreadLocalState();

	FTlsState		Prev;
};

////////////////////////////////////////////////////////////////////////////////
template <uint16 BufferSize>
TRedirectScope<BufferSize>::TRedirectScope()
{
	this->Head->ThreadId = ActiveGrowingRedirection;
	Prev = GTlsState;
	GTlsState.WriteBuffer = this->Head;
	GTlsState.NextBufferFn = &TRedirectScope::NextBuffer;
}

////////////////////////////////////////////////////////////////////////////////
template <uint16 BufferSize>
TRedirectScope<BufferSize>::~TRedirectScope()
{
	ResetThreadLocalState();
}

////////////////////////////////////////////////////////////////////////////////
template <uint16 BufferSize>
void TRedirectScope<BufferSize>::ResetThreadLocalState()
{
	if (Prev.WriteBuffer != nullptr)
	{
		GTlsState = Prev;
		Prev = {nullptr, nullptr};
	}
}

////////////////////////////////////////////////////////////////////////////////
template <uint16 BufferSize>
FWriteBuffer* TRedirectScope<BufferSize>::NextBuffer(FWriteBuffer* CurrentBuffer)
{
	FWriteBuffer* NextBuffer = TWriteBufferQueue<BufferSize>::NextBuffer(CurrentBuffer);
	NextBuffer->ThreadId = ActiveGrowingRedirection;
	return NextBuffer;
}

////////////////////////////////////////////////////////////////////////////////
template <uint16 BufferSize>
FWriteBuffer* TRedirectScope<BufferSize>::Finish()
{
	FWriteBuffer* FirstBuffer = this->Head;
	this->Head = nullptr;
	ResetThreadLocalState();
	return FirstBuffer;
}


////////////////////////////////////////////////////////////////////////////////

/**
 * Bounded redirect buffer.
 */
template <uint16 BufferSize>
class TBoundedRedirectScope
{
public:
	enum : uint16 { ActiveRedirection = 0xffff };

					TBoundedRedirectScope();
					~TBoundedRedirectScope();
	void			Close();
	void			Abandon();
	uint8*			GetData();
	uint32			GetSize() const;
	uint32			GetCapacity() const;
	void			Reset();

private:
	FTlsState		Prev;
	uint8			Data[BufferSize];
	FWriteBuffer	Buffer;
};

////////////////////////////////////////////////////////////////////////////////
template <uint16 BufferSize>
inline TBoundedRedirectScope<BufferSize>::TBoundedRedirectScope()
{
	Reset();
	Prev = GTlsState;
	GTlsState.WriteBuffer = &Buffer;
	GTlsState.NextBufferFn = nullptr;
	Buffer.Size = uint16(BufferSize);
	// When using the default NextBuffer function it looks out
	// for this thread id to avoid overflowing
	Buffer.ThreadId = ActiveRedirection;
}

////////////////////////////////////////////////////////////////////////////////
template <uint16 BufferSize>
inline TBoundedRedirectScope<BufferSize>::~TBoundedRedirectScope()
{
	Close();
}

////////////////////////////////////////////////////////////////////////////////
template <uint16 BufferSize>
inline void TBoundedRedirectScope<BufferSize>::Close()
{
	if (Buffer.ThreadId == ActiveRedirection)
	{
		Buffer.ThreadId = 0;
		GTlsState = Prev;
		Prev = {nullptr, nullptr};
	}
}

////////////////////////////////////////////////////////////////////////////////
template <uint16 BufferSize>
inline void TBoundedRedirectScope<BufferSize>::Abandon()
{
	Close();
}

////////////////////////////////////////////////////////////////////////////////
template <uint16 BufferSize>
inline uint8* TBoundedRedirectScope<BufferSize>::GetData()
{
	return Buffer.Reaped;
}

////////////////////////////////////////////////////////////////////////////////
template <uint16 BufferSize>
inline uint32 TBoundedRedirectScope<BufferSize>::GetSize() const
{
	return uint32(Buffer.Committed - Buffer.Reaped);
}

////////////////////////////////////////////////////////////////////////////////
template <uint16 BufferSize>
inline uint32 TBoundedRedirectScope<BufferSize>::GetCapacity() const
{
	return BufferSize;
}

////////////////////////////////////////////////////////////////////////////////
template <uint16 BufferSize>
inline void TBoundedRedirectScope<BufferSize>::Reset()
{
	Buffer.CursorMaskInv = 0;
	Buffer.SetCursor(Data + sizeof(uint32));
	Buffer.Committed = Buffer.ReadCursor();
	Buffer.Reaped = Buffer.ReadCursor();
}


////////////////////////////////////////////////////////////////////////////////

// Default size for fallback allocations. These are temporary
// allocations so fewer larger allocations makes sense.
// Should be a 128 byte multiple close to max value of uint16.
constexpr uint16 OnConnectBufferSize = 65408;

/**
 * Specialized redirect buffer queue for OnConnect callback.
 */
class FOnConnectRedirectScope : public TWriteBufferQueue<OnConnectBufferSize>
{
public:
	FOnConnectRedirectScope();
	~FOnConnectRedirectScope();
	static FWriteBuffer* GetBuffer();
	static FWriteBuffer* NextBuffer(FWriteBuffer* CurrentBuffer);

private:
	static FWriteBuffer* Tail;
};

////////////////////////////////////////////////////////////////////////////////
inline FOnConnectRedirectScope::FOnConnectRedirectScope()
{
	Tail = Head;
}

////////////////////////////////////////////////////////////////////////////////
inline FOnConnectRedirectScope::~FOnConnectRedirectScope()
{
	Tail = nullptr;
}

////////////////////////////////////////////////////////////////////////////////
inline FWriteBuffer* FOnConnectRedirectScope::GetBuffer()
{
	return Tail;
}


} // namespace Private
} // namespace Trace
} // namespace UE
  
#endif // TRACE_PRIVATE_MINIMAL_ENABLED
