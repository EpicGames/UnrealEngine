// Copyright Epic Games, Inc. All Rights Reserved.

#include "Trace/Config.h"
#include "Trace/Lock.h"

#if TRACE_PRIVATE_MINIMAL_ENABLED

#include "Message.h"
#include "Platform.h"
#include "ThreadRegistry.h"
#include "Trace/Detail/Atomic.h"
#include "Trace/Detail/Channel.h"
#include "Trace/Detail/Protocol.h"
#include "Trace/Detail/Transport.h"
#include "Trace/Trace.inl"
#include "WriteBufferRedirect.h"

#include <chrono>
#include <limits.h>
#include <stdlib.h>

#if PLATFORM_WINDOWS
#	define TRACE_PRIVATE_STOMP 0 // 1=overflow, 2=underflow
#	if TRACE_PRIVATE_STOMP
#	include "Windows/AllowWindowsPlatformTypes.h"
#	include "Windows/HideWindowsPlatformTypes.h"
#	endif
#else
#	define TRACE_PRIVATE_STOMP 0
#endif

#ifndef TRACE_PRIVATE_BUFFER_SEND
#	define TRACE_PRIVATE_BUFFER_SEND 0
#endif


namespace UE {
namespace Trace {
namespace Private {

////////////////////////////////////////////////////////////////////////////////
int32			EncodeInstr(const void*, int32, void*, int32);
int32			Encode(const void*, int32, void*, int32);
void			Writer_SetUpdateCallback(OnUpdateFunc* Callback);
void			Writer_SendData(uint32, uint8* __restrict, uint32);
void			Writer_InitializeTail(int32);
void			Writer_ShutdownTail();
void			Writer_TailOnConnect(uint32 MaxSize = 0);
#if TRACE_PRIVATE_ALLOW_IMPORTANTS
void			Writer_InitializeSharedBuffers();
void			Writer_ShutdownSharedBuffers();
void			Writer_UpdateSharedBuffers();
void			Writer_InitializeCache();
void			Writer_ShutdownCache();
void			Writer_CacheOnConnect();
#endif
void			Writer_CallbackOnConnect();
void			Writer_InitializePool();
void			Writer_ShutdownPool();
void			Writer_DrainBuffers();
void			Writer_DrainLocalBuffers();
void			Writer_DrainBufferList(uint32, FWriteBuffer*);
void			Writer_EndThreadBuffer();
uint32			Writer_GetControlPort();
void			Writer_UpdateControl();
void			Writer_InitializeControl();
void			Writer_ShutdownControl();
bool			Writer_IsTailing();
static bool		Writer_SessionPrologue();
void			Writer_FreeBlockListToPool(FWriteBuffer*, FWriteBuffer*);
void			Writer_SetBlockPoolLimit(uint32);
void			Writer_UnsetBlockPoolLimit();
void			Writer_ClearPendingHandle();
static void		Writer_Close();
static void		Writer_CloseNoFlush();

////////////////////////////////////////////////////////////////////////////////
void Writer_LogThreadGroupBegin(const ANSICHAR*, uint32);
void Writer_LogThreadGroupEnd();
void Writer_LogThreadInfo(uint32, uint32, int32, const ANSICHAR*, uint32);

////////////////////////////////////////////////////////////////////////////////
struct FTraceGuid
{
	uint32 Bits[4];
};

////////////////////////////////////////////////////////////////////////////////
UE_TRACE_MINIMAL_EVENT_BEGIN($Trace, NewTrace, NoSync)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, StartCycle)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, CycleFrequency)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint16, Endian)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint8, PointerSize)
	UE_TRACE_MINIMAL_EVENT_FIELD(double, StartDateTime)
UE_TRACE_MINIMAL_EVENT_END()

UE_TRACE_MINIMAL_EVENT_BEGIN($Trace, BandwidthStats)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint32, DeltaBytesSent)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint32, DeltaBytesEmitted)
UE_TRACE_MINIMAL_EVENT_END()

UE_TRACE_MINIMAL_EVENT_BEGIN($Trace, MemoryStats)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_MINIMAL_EVENT_FIELD(float,	 BlockPoolUsage)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, MemoryUsed)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, BlockPoolAllocated)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint32, FixedBufferAllocated)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint32, CacheAllocated)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint32, SharedAllocated)
UE_TRACE_MINIMAL_EVENT_END()

////////////////////////////////////////////////////////////////////////////////
static volatile bool			GInitialized;		// = false;
FStatistics						GTraceStatistics;	// = {};
static double					GStartDateTime;     // = 0;
TRACELOG_API uint64				GStartCycle;		// = 0;
TRACELOG_API uint32 volatile	GLogSerial;			// = 0;
// Counter of calls to Writer_WorkerUpdate to enable regular flushing of output buffers
static uint32					GUpdateCounter;		// = 0;
static uint32					GBlockPoolMaxSize;
#if UE_TRACE_PACKET_VERIFICATION
uint64							GPacketSerial = 1;
#endif

////////////////////////////////////////////////////////////////////////////////
struct FWriterState
{
	IoWriteFunc* Write = nullptr;
	IoCloseFunc* Close = nullptr;
	IoUpdateFunc* Update = nullptr;
	IoIsReadyFunc* IsReady = nullptr;
};

static FWriterState GWriterState;
static FWriterState GPendingWriterState;
static uint8 GPendingWriterLock;


////////////////////////////////////////////////////////////////////////////////
// When a thread terminates we want to recover its trace buffer. To do this we
// use a thread_local object whose destructor gets called as the thread ends. On
// some C++ standard library implementations this is implemented using a thread-
// specific atexit() call and can involve taking a lock. As tracing can start
// very early or often be used during shared-object loads, there is a risk of
// a deadlock initialising the context object. The define below can be used to
// implement alternatives via a 'ThreadOnThreadExit()' symbol.
#if !defined(UE_TRACE_USE_TLS_CONTEXT_OBJECT)
#	define UE_TRACE_USE_TLS_CONTEXT_OBJECT 1
#endif

#if UE_TRACE_USE_TLS_CONTEXT_OBJECT

////////////////////////////////////////////////////////////////////////////////
struct FWriteTlsContext
{
				~FWriteTlsContext();
	uint32		GetThreadId();

private:
	uint32		ThreadId = 0;
};

////////////////////////////////////////////////////////////////////////////////
FWriteTlsContext::~FWriteTlsContext()
{
	if (AtomicLoadRelaxed(&GInitialized))
	{
		Writer_EndThreadBuffer();
	}
}

////////////////////////////////////////////////////////////////////////////////
uint32 FWriteTlsContext::GetThreadId()
{
	if (ThreadId)
	{
		return ThreadId;
	}

	static uint32 volatile Counter;
	ThreadId = AtomicAddRelaxed(&Counter, 1u) + ETransportTid::Bias;
	return ThreadId;
}

////////////////////////////////////////////////////////////////////////////////
thread_local FWriteTlsContext	GTlsContext;

////////////////////////////////////////////////////////////////////////////////
uint32 Writer_GetThreadId()
{
	return GTlsContext.GetThreadId();
}

#else // UE_TRACE_USE_TLS_CONTEXT_OBJECT

////////////////////////////////////////////////////////////////////////////////
void ThreadOnThreadExit(void (*)());

////////////////////////////////////////////////////////////////////////////////
uint32 Writer_GetThreadId()
{
	static thread_local uint32 ThreadId;
	if (ThreadId)
	{
		return ThreadId;
	}

	ThreadOnThreadExit([] () { Writer_EndThreadBuffer(); });

	static uint32 volatile Counter;
	ThreadId = AtomicAddRelaxed(&Counter, 1u) + ETransportTid::Bias;
	return ThreadId;
}

#endif // UE_TRACE_USE_TLS_CONTEXT_OBJECT

////////////////////////////////////////////////////////////////////////////////
uint64 TimeGetRelativeTimestamp()
{
	return GStartCycle ? TimeGetTimestamp() - GStartCycle : 0;
}

////////////////////////////////////////////////////////////////////////////////
void Writer_CreateGuid(FTraceGuid* OutGuid)
{
	// This is not thread safe. Should only be accessed from the writer thread.
	// This initialized the prng with the current timestamp. In theory two machines could initialize on the exact same time
	// producing the same sequence of guids.
	static uint64 State = TimeGetTimestamp();
	// L'Ecuyer, Pierre (1999). "Tables of Linear Congruential Generators of Different Sizes and Good Lattice Structure"
	// corrected with errata
	// Assuming m = 2e64
	constexpr uint64 C = 0x369DEA0F31A53F85;
	constexpr uint64 I = 1ull;

	const uint64 TopBits = State * C + I;
	const uint64 BottomBits = TopBits * C + I;
	State = BottomBits;

	*(uint64*)&OutGuid->Bits[0] = TopBits;
	*(uint64*)&OutGuid->Bits[2] = BottomBits;

	constexpr uint8 Version = 0x40; //Version 4, 4 bits
	constexpr uint8 VersionMask = 0xf0;
	constexpr uint8 Variant = 0x80; //Variant 1, 2 bits
	constexpr uint8 VariantMask = 0xc0;

	uint8* Octets = (uint8*)OutGuid;
	Octets[6] = Version | (~VersionMask & Octets[6]); // Octet 9
	Octets[8] = Variant | (~VariantMask & Octets[8]); // Octet 7
}

////////////////////////////////////////////////////////////////////////////////

OnScopeBeginFunc* FProfilerScope::OnScopeBegin = nullptr;
OnScopeEndFunc* FProfilerScope::OnScopeEnd = nullptr;

////////////////////////////////////////////////////////////////////////////////
void*			(*AllocHook)(SIZE_T, uint32);			// = nullptr
void			(*FreeHook)(void*, SIZE_T);				// = nullptr

////////////////////////////////////////////////////////////////////////////////
void Writer_MemorySetHooks(decltype(AllocHook) Alloc, decltype(FreeHook) Free)
{
	AllocHook = Alloc;
	FreeHook = Free;
}

////////////////////////////////////////////////////////////////////////////////
void* Writer_MemoryAllocate(SIZE_T Size, uint32 Alignment)
{
	void* Ret = nullptr;

#if TRACE_PRIVATE_STOMP
	static uint8* Base;
	if (Base == nullptr)
	{
		Base = (uint8*)VirtualAlloc(0, 1ull << 40, MEM_RESERVE, PAGE_READWRITE);
	}

	static SIZE_T PageSize = 4096;
	Base += PageSize;
	uint8* NextBase = Base + ((PageSize - 1 + Size) & ~(PageSize - 1));
	VirtualAlloc(Base, SIZE_T(NextBase - Base), MEM_COMMIT, PAGE_READWRITE);
#if TRACE_PRIVATE_STOMP == 1
	Ret = NextBase - Size;
#elif TRACE_PRIVATE_STOMP == 2
	Ret = Base;
#endif
	Base = NextBase;
#else // TRACE_PRIVATE_STOMP

	if (AllocHook != nullptr)
	{
		Ret = AllocHook(Size, Alignment);
	}
	else
	{
#if defined(_MSC_VER)
		Ret = _aligned_malloc(Size, Alignment);
#elif (defined(__ANDROID_API__) && __ANDROID_API__ < 28) || defined(__APPLE__)
		int Result = posix_memalign(&Ret, Alignment, Size);
		if (Result != 0)
		{
			UE_TRACE_MESSAGE_F(OOMFatal, "Failed to allocate %llu bytes with alignment %u: %d", uint64(Size), Alignment, Result);
		}
#else
		Ret = aligned_alloc(Alignment, Size);
#endif
	}
#endif // TRACE_PRIVATE_STOMP

	if (Ret == nullptr)
	{
		UE_TRACE_MESSAGE_F(OOMFatal, "OOM allocating %llu bytes", uint64(Size));
	}

#if TRACE_PRIVATE_STATISTICS
	AtomicAddRelaxed(&GTraceStatistics.MemoryUsed, uint64(Size));
#endif

	return Ret;
}

////////////////////////////////////////////////////////////////////////////////
void Writer_MemoryFree(void* Address, uint32 Size)
{
#if TRACE_PRIVATE_STOMP
	if (Address == nullptr)
	{
		return;
	}

	*(uint8*)Address = 0xfe;

	MEMORY_BASIC_INFORMATION MemInfo;
	VirtualQuery(Address, &MemInfo, sizeof(MemInfo));

	DWORD Unused;
	VirtualProtect(MemInfo.BaseAddress, MemInfo.RegionSize, PAGE_READONLY, &Unused);
#else // TRACE_PRIVATE_STOMP
	if (FreeHook != nullptr)
	{
		FreeHook(Address, Size);
	}
	else
	{
#if defined(_MSC_VER)
		_aligned_free(Address);
#else
		free(Address);
#endif
	}
#endif // TRACE_PRIVATE_STOMP

#if TRACE_PRIVATE_STATISTICS
	AtomicAddRelaxed(&GTraceStatistics.MemoryUsed, uint64(-int64(Size)));
#endif
}



////////////////////////////////////////////////////////////////////////////////
static UPTRINT					GDataHandle;		// = 0
static volatile UPTRINT			GPendingDataHandle;	// = 0

////////////////////////////////////////////////////////////////////////////////
#if TRACE_PRIVATE_BUFFER_SEND
static const SIZE_T GSendBufferSize = 1 << 20; // 1Mb
uint8* GSendBuffer; // = nullptr;
uint8* GSendBufferCursor; // = nullptr;
static bool Writer_FlushSendBuffer()
{
	if( GDataHandle && GSendBufferCursor > GSendBuffer )
	{
		if (!GWriterState.Write(GDataHandle, GSendBuffer, GSendBufferCursor - GSendBuffer))
		{
			UE_TRACE_ERRORMESSAGE(WriteError, GetLastErrorCode());
			Writer_CloseNoFlush(); // Don't flush or we'll recurse
			return false;
		}
		GSendBufferCursor = GSendBuffer;
	}
	return true;
}
#else
static bool Writer_FlushSendBuffer() { return true; }
#endif

////////////////////////////////////////////////////////////////////////////////
static void Writer_SendDataImplNoInstr(const void* Data, uint32 Size)
{
#if TRACE_PRIVATE_STATISTICS
	FPlatformAtomics::AtomicStore_Relaxed((int64*)&GTraceStatistics.BytesSent, FPlatformAtomics::AtomicRead_Relaxed((int64*)&GTraceStatistics.BytesSent) + Size);
#endif

#if TRACE_PRIVATE_BUFFER_SEND
	// If there's not enough space for this data, flush
	if (GSendBufferCursor + Size > GSendBuffer + GSendBufferSize)
	{
		if (!Writer_FlushSendBuffer())
		{
			return;
		}
	}

	// Should rarely happen but if we're asked to send large data send it directly
	if (Size > GSendBufferSize)
	{
		if (!GWriterState.Write(GDataHandle, Data, Size))
		{
			Writer_CloseNoFlush();
		}
	}
	// Otherwise append to the buffer
	else
	{
		memcpy(GSendBufferCursor, Data, Size);
		GSendBufferCursor += Size;
	}
#else
	if (!GWriterState.Write(GDataHandle, Data, Size))
	{
		UE_TRACE_ERRORMESSAGE(WriteError, GetLastErrorCode());
		Writer_CloseNoFlush();
	}
#endif
}

////////////////////////////////////////////////////////////////////////////////
static void Writer_SendDataImpl(const void* Data, uint32 Size)
{
	FProfilerScope _(__func__);
	Writer_SendDataImplNoInstr(Data, Size);
}

////////////////////////////////////////////////////////////////////////////////
void Writer_SendDataRaw(const void* Data, uint32 Size)
{
	if (!GDataHandle)
	{
		return;
	}

	FProfilerScope _(__func__);
	Writer_SendDataImplNoInstr(Data, Size);
}

////////////////////////////////////////////////////////////////////////////////
void Writer_SendDataNoInstr(uint32 ThreadId, uint8* __restrict Data, uint32 Size)
{
	static_assert(ETransport::Active == ETransport::TidPacketSync, "Active should be set to what the compiled code uses. It is used to track places that assume transport packet format");

	if (!GDataHandle)
	{
		return;
	}

	constexpr uint32 MaxEncodedBuffer = UE_TRACE_BLOCK_SIZE;
	constexpr uint32 EncodingOverhead = 64;

	// Smaller buffers usually aren't redundant enough to benefit from being
	// compressed. They often end up being larger.
	if (Size > 384 && Size <= MaxEncodedBuffer)
	{
		// Buffer size is expressed as "A + B" where A is a maximum expected
		// input size (i.e. at least GPoolBlockSize) and B is LZ4 overhead as
		// per LZ4_COMPRESSBOUND.
		TTidPacketEncoded<MaxEncodedBuffer + EncodingOverhead> Packet;

		Packet.DecodedSize = uint16(Size);
		Packet.PacketSize = uint16(Encode(Data, Packet.DecodedSize, Packet.Data, sizeof(Packet.Data)));

		if (Packet.PacketSize > 0 &&
			Packet.PacketSize < (Packet.DecodedSize + EncodingOverhead))
		{
			Packet.ThreadId = FTidPacketBase::EncodedMarker;
			Packet.ThreadId |= uint16(ThreadId & FTidPacketBase::ThreadIdMask);
#if UE_TRACE_PACKET_VERIFICATION
			Packet.ThreadId |= FTidPacketBase::Verification;
#endif
			Packet.PacketSize += sizeof(FTidPacketEncoded);

			Writer_SendDataImplNoInstr(&Packet, Packet.PacketSize);

#if UE_TRACE_PACKET_VERIFICATION
			const uint64 Serial = GPacketSerial++;
			Writer_SendDataImplNoInstr(&Serial, sizeof(uint64));
#endif
			return;
		}
		else
		{
			//PLATFORM_BREAK();
			// Intentional fall through to write uncompressed data...
		}
	}

	// Write uncompressed data.
	{
		Data -= sizeof(FTidPacket);
		Size += sizeof(FTidPacket);
		auto* Packet = (FTidPacket*)Data;
		Packet->ThreadId = uint16(ThreadId & FTidPacketBase::ThreadIdMask);
#if UE_TRACE_PACKET_VERIFICATION
		Packet->ThreadId |= FTidPacketBase::Verification;
#endif
		Packet->PacketSize = uint16(Size);

		Writer_SendDataImplNoInstr(Data, Size);

#if UE_TRACE_PACKET_VERIFICATION
		const uint64 Serial = GPacketSerial++;
		Writer_SendDataImplNoInstr(&Serial, sizeof(uint64));
#endif
	}
}

////////////////////////////////////////////////////////////////////////////////
void Writer_SendData(uint32 ThreadId, uint8* __restrict Data, uint32 Size)
{
	if (!GDataHandle)
	{
		return;
	}

	FProfilerScope _(__func__);

	constexpr uint32 MaxEncodedBuffer = UE_TRACE_BLOCK_SIZE;
	constexpr uint32 EncodingOverhead = 64;

	// Smaller buffers usually aren't redundant enough to benefit from being
	// compressed. They often end up being larger. Also if the buffer is too
	// large for the fixed buffer, try sending it uncompressed.
	if (Size > 384 && Size <= MaxEncodedBuffer)
	{
		// Buffer size is expressed as "A + B" where A is a maximum expected
		// input size (i.e. at least GPoolBlockSize) and B is LZ4 overhead as
		// per LZ4_COMPRESSBOUND.
		TTidPacketEncoded<MaxEncodedBuffer + EncodingOverhead> Packet;

		Packet.DecodedSize = uint16(Size);
		Packet.PacketSize = uint16(EncodeInstr(Data, Packet.DecodedSize, Packet.Data, sizeof(Packet.Data)));

		if (Packet.PacketSize > 0 &&
			Packet.PacketSize < (Packet.DecodedSize + EncodingOverhead))
		{
			Packet.ThreadId = uint16(ThreadId & FTidPacketBase::ThreadIdMask);
			Packet.ThreadId |= FTidPacketBase::EncodedMarker;
#if UE_TRACE_PACKET_VERIFICATION
			Packet.ThreadId |= FTidPacketBase::Verification;
#endif
			Packet.PacketSize += sizeof(FTidPacketEncoded);

			Writer_SendDataImpl(&Packet, Packet.PacketSize);
#if UE_TRACE_PACKET_VERIFICATION
			const uint64 Serial = GPacketSerial++;
			Writer_SendDataImpl(&Serial, sizeof(uint64));
#endif
			return;
		}
		else
		{
			//PLATFORM_BREAK();
			// Intentional fall through to write uncompressed data...
		}
	}

	// Write uncompressed data.
	{
		// When setting up the buffer we have left space for
		// a uncompressed packet header ahead of the data.
		Data -= sizeof(FTidPacket);
		Size += sizeof(FTidPacket);
		auto* Packet = (FTidPacket*)Data;
		Packet->ThreadId = uint16(ThreadId & FTidPacketBase::ThreadIdMask);
#if UE_TRACE_PACKET_VERIFICATION
		Packet->ThreadId |= FTidPacketBase::Verification;
#endif
		Packet->PacketSize = uint16(Size);

		Writer_SendDataImpl(Data, Size);

#if UE_TRACE_PACKET_VERIFICATION
		const uint64 Serial = GPacketSerial++;
		Writer_SendDataImpl(&Serial, sizeof(uint64));
#endif
	}
}
////////////////////////////////////////////////////////////////////////////////
static void Writer_DescribeEvents(FEventNode::FIter Iter)
{
	FProfilerScope _(__func__);

	// Since we are using on stack buffer to collect the describing events we
	// cannot emit additional data on this thread. So use the un-instrumented
	// send data functions here.
	TBoundedRedirectScope<4096> TraceData;
	// Defines the maximum size an event description is allowed to take. This
	// controls the minimal available headroom in the above buffer.
	constexpr uint32 MaxEventDescBytes = 1024;

	while (const FEventNode* Event = Iter.GetNext())
	{
		Event->Describe();

		// Flush just in case an NewEvent event will be larger than MaxEventDescBytes.
		uint32 RedirectSize = TraceData.GetSize();
		if (RedirectSize >= (TraceData.GetCapacity() - MaxEventDescBytes))
		{
			Writer_SendDataNoInstr(ETransportTid::Events, TraceData.GetData(), RedirectSize);
			if (TraceData.GetSize() != RedirectSize)
			{
				// Trace events were emitted during Writer_SendDataNoInstr() call.
				// These events will be lost!
				//PLATFORM_BREAK();
				UE_TRACE_MESSAGE(WriterError, "Events emitted while describing events; trace will be corrupt!");
			}
			TraceData.Reset();
		}
	}

	if (uint32 RedirectSize = TraceData.GetSize())
	{
		Writer_SendDataNoInstr(ETransportTid::Events, TraceData.GetData(), RedirectSize);
		if (TraceData.GetSize() != RedirectSize)
		{
			// Trace events were emitted during Writer_SendDataNoInstr() call.
			// These events will be lost!
			// PLATFORM_BREAK();
			UE_TRACE_MESSAGE(WriterError, "Events emitted while describing events; trace will be corrupt!");
		}
	}
}

////////////////////////////////////////////////////////////////////////////////
static void Writer_AnnounceChannels()
{
	FProfilerScope _(__func__);
	FChannel::Iter Iter = FChannel::ReadNew();
	while (const FChannel* Channel = Iter.GetNext())
	{
		Channel->Announce();
	}
}

////////////////////////////////////////////////////////////////////////////////
static void Writer_DescribeAnnounce()
{
	if (!GDataHandle)
	{
		return;
	}

	Writer_AnnounceChannels();
	Writer_DescribeEvents(FEventNode::ReadNew());
}

////////////////////////////////////////////////////////////////////////////////
void EmitThreadGroups(volatile FThreadGroupNode* Group)
{
    // Emit GroupBegin
	Writer_LogThreadGroupBegin((const ANSICHAR*)Group->GroupName, (uint32)strlen((const ANSICHAR*)Group->GroupName) + 1);

	// Emit ThreadInfo for each thread in this group
	for (volatile FThreadInfoNode* Thread = AtomicLoadRelaxed(&Group->ThreadInfoHead); Thread != nullptr; Thread = AtomicLoadRelaxed(&Thread->Next))
	{
		Writer_LogThreadInfo(Thread->ThreadId, Thread->SystemId, Thread->SortHint, (const ANSICHAR*)Thread->Name, (uint32)strlen((const ANSICHAR*)Thread->Name) + 1);
	}

	// Emit GroupEnd if we know this group has ended
	volatile FThreadGroupNode* CurrentGroup = Writer_GetCurrentGroup();
    if (Group != CurrentGroup)
    {
	    Writer_LogThreadGroupEnd();
    }
}

////////////////////////////////////////////////////////////////////////////////
void EmitUngroupedThreads(volatile FThreadInfoNode* Thread)
{
	Writer_LogThreadInfo(Thread->ThreadId, Thread->SystemId, Thread->SortHint, (const ANSICHAR*)Thread->Name, (uint32)strlen((const ANSICHAR*)Thread->Name) + 1);
}

////////////////////////////////////////////////////////////////////////////////
static void Writer_EnumerateThreadEventsOnConnect()
{
	FProfilerScope _(__func__);

	// Enumerate ungrouped first as the last thread group emitted may still be open.
	Writer_EnumerateUngroupedThreads(&EmitUngroupedThreads);
	Writer_EnumerateThreadGroups(&EmitThreadGroups);
}

////////////////////////////////////////////////////////////////////////////////
static int8				GSyncPacketCountdown; // = 0
static const int8		GNumSyncPackets = 3;
static OnConnectFunc*	GOnConnection = nullptr;
static OnCloseFunc*		GOnClose = nullptr;
static OnUpdateFunc*	GOnUpdate = nullptr;
static FTraceGuid		GSessionGuid; // = {0, 0, 0, 0};
static FTraceGuid		GTraceGuid; // = {0, 0, 0, 0};
static const float		GMemoryStatsFrequencySeconds = 1.0f; // Every second

////////////////////////////////////////////////////////////////////////////////
static void Writer_TraceStats(const FStatistics& InStats)
{
#if UE_TRACE_EMIT_STATISTICS
	static uint64 LastMemoryStatsCycle = 0;
	static uint64 LastSent = InStats.BytesSent;
	static uint64 LastTraced = InStats.BytesTraced;
	static uint64 LastEmitted = InStats.BytesEmitted;

	// Only send stats when there is an actual connection, so we
	// don't use the tail buffer unnecessarily.
	if (GDataHandle)
	{
		const uint64 CurrentTimeCycle = TimeGetRelativeTimestamp();
		const float BlockPoolFreeFraction = InStats.BlockPoolAllocatedBlocks ?
			float(InStats.BlockPoolFreeBlocks) / float(InStats.BlockPoolAllocatedBlocks) : 1.0f;
		const float BlockPoolUsage = 1.0f - BlockPoolFreeFraction;
		const uint32 DeltaEmitted = uint32(InStats.BytesEmitted - LastEmitted);
		uint32 DeltaSent = uint32(InStats.BytesSent - LastSent);
		if (InStats.BytesSent < LastSent)
		{
			// BytesSent is reset on every new connection
			DeltaSent = uint32(InStats.BytesSent);
		}

		// Send bandwidth stats every update
		UE_TRACE_MINIMAL_LOG($Trace, BandwidthStats, TraceLogChannel)
			<< BandwidthStats.Cycle(CurrentTimeCycle)
			<< BandwidthStats.DeltaBytesSent(DeltaSent)
			<< BandwidthStats.DeltaBytesEmitted(DeltaEmitted);

		// Only send memory stats on configured frequency
		const float MemoryStatDeltaSeconds = float(CurrentTimeCycle - LastMemoryStatsCycle) / float(TimeGetFrequency());
		if (MemoryStatDeltaSeconds > GMemoryStatsFrequencySeconds)
		{
			UE_TRACE_MINIMAL_LOG($Trace, MemoryStats, TraceLogChannel)
				<< MemoryStats.Cycle(CurrentTimeCycle)
				<< MemoryStats.MemoryUsed(InStats.MemoryUsed)
				<< MemoryStats.BlockPoolAllocated(InStats.BlockPoolAllocated)
				<< MemoryStats.FixedBufferAllocated(InStats.FixedBufferAllocated)
				<< MemoryStats.CacheAllocated(InStats.CacheAllocated)
				<< MemoryStats.SharedAllocated(InStats.SharedBufferAllocated)
				<< MemoryStats.BlockPoolUsage(BlockPoolUsage);

			LastMemoryStatsCycle = CurrentTimeCycle;
		}
	}

	LastSent = InStats.BytesSent;
	LastTraced = InStats.BytesTraced;
	LastEmitted = InStats.BytesEmitted;
#endif
}

////////////////////////////////////////////////////////////////////////////////
static void Writer_SendSync()
{
	if (GSyncPacketCountdown <= 0 || !GDataHandle)
	{
		return;
	}

	// It is possible that some events get collected and discarded by a previous
	// update that are newer than events sent it the following update where IO
	// is established. This will result in holes in serial numbering. A few sync
	// points are sent to aid analysis in determining what are holes and what is
	// just a requirement for more data. Holes will only occur at the start.

	// Note that Sync is alias as Important/Internal as changing Bias would
	// break backwards compatibility.

	FTidPacketBase SyncPacket = { sizeof(SyncPacket), ETransportTid::Sync };
	Writer_SendDataImpl(&SyncPacket, sizeof(SyncPacket));

	--GSyncPacketCountdown;
}

////////////////////////////////////////////////////////////////////////////////
static void Writer_Close()
{
	Writer_FlushSendBuffer();
	Writer_CloseNoFlush();
}

////////////////////////////////////////////////////////////////////////////////
static void Writer_CloseNoFlush()
{
	if (GDataHandle)
	{
		GWriterState.Close(GDataHandle);
		if (GOnClose)
		{
			GOnClose();
		}
	}
	GDataHandle = 0;
}

////////////////////////////////////////////////////////////////////////////////
static bool Writer_UpdateConnection()
{
	UPTRINT PendingDataHandle = AtomicLoadRelaxed(&GPendingDataHandle);

	// Update active connection if an update function has been set
	if (GDataHandle && GWriterState.Update != nullptr)
	{
		if (!GWriterState.Update(GDataHandle))
		{
			Writer_CloseNoFlush();
		}
	}

	if (!PendingDataHandle)
	{
		return false;
	}

	FProfilerScope _(__func__);
	
	// Pending writer state is guaranteed to be written if the
	// pending data handle is set. See Writer_SetPendingHandle
	FWriterState PendingWriterState = GPendingWriterState;

	// Is this a close request? So that we capture some of the events around
	// the closure we will add some inertia before enacting the close.
	static int32 CloseInertia = 0;
	if (PendingDataHandle == ~UPTRINT(0))
	{
		if (CloseInertia <= 0)
			CloseInertia = 2;

		--CloseInertia;
		if (CloseInertia <= 0)
		{
			Writer_Close();
			AtomicStoreRelaxed(&GPendingDataHandle, UPTRINT(0));
		}

		return true;
	}
	
	// Extract send flags
	uint32 SendFlags = uint32(PendingDataHandle >> 48ull);
	PendingDataHandle &= 0x0000'ffff'ffff'ffffull;

	// Reject the pending connection if we've already got a connection
	if (GDataHandle)
	{
		PendingWriterState.Close(PendingDataHandle);
		Writer_ClearPendingHandle();
		return false;
	}

	// Update the pending connection
	if (PendingWriterState.Update != nullptr)
	{
		if(!PendingWriterState.Update(PendingDataHandle))
		{
			PendingWriterState.Close(PendingDataHandle);
			Writer_ClearPendingHandle();
			return false;
		}
	}
	// Check if the pending connection is ready
	if (PendingWriterState.IsReady != nullptr && !PendingWriterState.IsReady(PendingDataHandle))
	{
		return false;
	}

	// Now clear pending handle, we're committing to this as the new active connection
	Writer_ClearPendingHandle();

	// Generate Guid for new connection
	Writer_CreateGuid(&GTraceGuid);

	GWriterState = PendingWriterState;
	AtomicStoreRelease(&GDataHandle, PendingDataHandle);
	if (!Writer_SessionPrologue())
	{
		return false;
	}

	// Reset statistics.
	GTraceStatistics.BytesSent = 0;

	// The first events we will send are ones that describe the trace's events
	FEventNode::OnConnect();
	Writer_DescribeEvents(FEventNode::ReadNew());

	// Send cached events (i.e. importants)
#if TRACE_PRIVATE_ALLOW_IMPORTANTS
	Writer_CacheOnConnect();
#endif

	// Issue on connection callback. This allows writing events that are
	// not cached but important for the cache
	Writer_CallbackOnConnect();

	// Finally write the events in the tail buffer
	if ((SendFlags & FSendFlags::ExcludeTail) == 0)
	{
		Writer_TailOnConnect();
	}

	// See Writer_SendSync() for details.
	GSyncPacketCountdown = GNumSyncPackets;

	return true;
}

////////////////////////////////////////////////////////////////////////////////
static bool Writer_SessionPrologue()
{
	if (!GDataHandle)
	{
		return false;
	}

#if TRACE_PRIVATE_BUFFER_SEND
	if (!GSendBuffer)
	{
		GSendBuffer = static_cast<uint8*>(Writer_MemoryAllocate(GSendBufferSize, 16));
#if TRACE_PRIVATE_STATISTICS
		AtomicAddRelaxed(&GTraceStatistics.FixedBufferAllocated, uint32(GSendBufferSize));
#endif
	}
	GSendBufferCursor = GSendBuffer;
#endif

	// Handshake.
	struct FHandshake
	{
		uint32 Magic			= '2' | ('C' << 8) | ('R' << 16) | ('T' << 24);
		uint16 MetadataSize		= uint16(MetadataSizeSum);
		uint16 MetadataField0	= uint16(sizeof(ControlPort) | (ControlPortFieldId << 8));
		uint16 ControlPort		= uint16(Writer_GetControlPort());
		uint16 MetadataField1	= uint16(sizeof(FTraceGuid) | (SessionGuidFieldId << 8));
		uint8 SessionGuid[16];	// Avoid padding
		uint16 MetadataField2	= uint16(sizeof(FTraceGuid) | (TraceGuidFieldId << 8));
		uint8 TraceGuid[16];	// Avoid padding
		enum
		{
			MetadataSizeSum		= 2 + 2 + 2 + 16 + 2 + 16,
			Size				= MetadataSizeSum + 4 + 2,
			ControlPortFieldId	= 0,
			SessionGuidFieldId	= 1,
			TraceGuidFieldId	= 2,
		};
	};
	FHandshake Handshake;
	memcpy(&Handshake.SessionGuid, &GSessionGuid, sizeof(FTraceGuid));
	memcpy(&Handshake.TraceGuid, &GTraceGuid, sizeof(FTraceGuid));
	bool bOk = GWriterState.Write(GDataHandle, &Handshake, FHandshake::Size);

	// Stream header
	const struct {
		uint8 TransportVersion	= ETransport::TidPacketSync;
		uint8 ProtocolVersion	= EProtocol::Id;
	} TransportHeader;
	bOk &= GWriterState.Write(GDataHandle, &TransportHeader, sizeof(TransportHeader));

	// Transmit New Trace before any other events
	if (bOk)
	{
		TBoundedRedirectScope<64> NewTraceBuffer;
		UE_TRACE_MINIMAL_LOG($Trace, NewTrace, TraceLogChannel)
		   << NewTrace.StartCycle(GStartCycle)
		   << NewTrace.CycleFrequency(TimeGetFrequency())
		   << NewTrace.Endian(uint16(0x524d))
		   << NewTrace.PointerSize(uint8(sizeof(void*)))
		   << NewTrace.StartDateTime(GStartDateTime);

		if (uint32 Size = NewTraceBuffer.GetSize())
		{
			uint8* Data = NewTraceBuffer.GetData();

			Data -= sizeof(FTidPacket);
			Size += sizeof(FTidPacket);
			FTidPacket* Packet = (FTidPacket*)Data;
			Packet->ThreadId = uint16(Writer_GetThreadId() & FTidPacketBase::ThreadIdMask);
			Packet->PacketSize = uint16(Size);

			// Transmit as a single packet of data
			bOk &= GWriterState.Write(GDataHandle, Data, Size);
#if UE_TRACE_PACKET_VERIFICATION
			if (bOk)
			{
				const uint64 Serial = GPacketSerial++;
				bOk &= GWriterState.Write(GDataHandle, &Serial, sizeof(uint64));
			}
#endif
		}
	}

	if (!bOk)
	{
		UE_TRACE_ERRORMESSAGE(WriteError, GetLastErrorCode());
		Writer_CloseNoFlush();
		return false;
	}

	return true;
}

////////////////////////////////////////////////////////////////////////////////
void Writer_CallbackOnConnect()
{
	if (!GOnConnection)
	{
		return;
	}

	// In order for events to appear ahead of the tail emit
	// them on a special thread id.
	FOnConnectRedirectScope OnConnectEvents;

	// Issue callback. We assume any events emitted here are not marked as
	// important and emitted on this thread.
	GOnConnection();

	// Send thread events
	Writer_EnumerateThreadEventsOnConnect();

	// Write the contents of the redirect buffers to the stream
	Writer_DrainBufferList(ETransportTid::PseudoImportants, OnConnectEvents.Head);
}

////////////////////////////////////////////////////////////////////////////////
static UPTRINT			GWorkerThread;		// = 0;
static volatile bool	GWorkerThreadQuit;	// = false;
static uint32			GSleepTimeInMS = UE_TRACE_WRITER_SLEEP_MS;
static volatile uint32	GUpdateInProgress = 1;	// Don't allow updates until initialized

////////////////////////////////////////////////////////////////////////////////
void Writer_SetUpdateCallback(OnUpdateFunc* Callback)
{
	AtomicStoreRelaxed(&GOnUpdate, Callback);
}

////////////////////////////////////////////////////////////////////////////////
static void Writer_WorkerUpdateInternal()
{
	FProfilerScope _(__func__);

	Writer_UpdateControl();
	Writer_UpdateConnection();
	Writer_DescribeAnnounce();
#if TRACE_PRIVATE_ALLOW_IMPORTANTS
	Writer_UpdateSharedBuffers();
#endif
	Writer_DrainBuffers();
	Writer_TraceStats(GTraceStatistics);
	Writer_SendSync();

#if TRACE_PRIVATE_BUFFER_SEND
	const uint32 FlushSendBufferCadenceMask = 8-1; // Flush every 8 calls
	if((++GUpdateCounter & FlushSendBufferCadenceMask) == 0 && GDataHandle != 0)
	{
		Writer_FlushSendBuffer();
	}
#endif

	if (OnUpdateFunc* OnUpdate = AtomicLoadRelaxed(&GOnUpdate))
	{
		OnUpdate();
	}
}

////////////////////////////////////////////////////////////////////////////////
static void Writer_WorkerUpdate()
{
	if (!AtomicCompareExchangeAcquire(&GUpdateInProgress, 1u, 0u))
	{
		return;
	}

	Writer_WorkerUpdateInternal();

	AtomicExchangeRelease(&GUpdateInProgress, 0u);
}

////////////////////////////////////////////////////////////////////////////////
static void Writer_WorkerThread()
{
	ThreadRegister(TEXT("Trace"), 0, INT_MAX);

	// Don't set GBlockPoolMaxSize until the spawning thread is fully done
	while (!AtomicCompareExchangeAcquire(&GUpdateInProgress, 1u, 0u)) {
		if (AtomicLoadRelaxed(&GWorkerThreadQuit))
		{
			return;
		}
	}

	// Enable the pool limit when the worker thread is running
	Writer_SetBlockPoolLimit(GBlockPoolMaxSize);

	AtomicExchangeRelease(&GUpdateInProgress, 0u);

	const double TicksToMS = 1000.0 / (double)TimeGetFrequency();

	while (!AtomicLoadRelaxed(&GWorkerThreadQuit))
	{
		uint64 StartTime = TimeGetTimestamp();
		Writer_WorkerUpdate();
		uint64 EndTime = TimeGetTimestamp();
		uint32 DurationMS = uint32(double(EndTime - StartTime) * TicksToMS);

		ThreadSleep(DurationMS < GSleepTimeInMS ? GSleepTimeInMS - DurationMS : 0);
	}

	// Reset the limit as no one will pick up data
	Writer_UnsetBlockPoolLimit();
}

////////////////////////////////////////////////////////////////////////////////
void Writer_WorkerCreate()
{
	if (GWorkerThread || AtomicLoadRelaxed(&GWorkerThreadQuit))
	{
		return;
	}

	GWorkerThread = ThreadCreate("TraceWorker", Writer_WorkerThread);
}

////////////////////////////////////////////////////////////////////////////////
static void Writer_WorkerJoin()
{
	if (!GWorkerThread)
	{
		return;
	}

	AtomicStoreRelaxed(&GWorkerThreadQuit, true);

	ThreadJoin(GWorkerThread);
	ThreadDestroy(GWorkerThread);

	Writer_WorkerUpdate();

	GWorkerThread = 0;
}



////////////////////////////////////////////////////////////////////////////////
static void Writer_InternalInitializeImpl()
{
	if (AtomicLoadRelaxed(&GInitialized))
	{
		return;
	}

	GStartDateTime = std::chrono::duration_cast<std::chrono::duration<double>>
		(std::chrono::system_clock::now().time_since_epoch()).count();

	GStartCycle = TimeGetTimestamp();

#if TRACE_PRIVATE_ALLOW_IMPORTANTS
	Writer_InitializeSharedBuffers();
#endif
	Writer_InitializePool();
	Writer_InitializeControl();

	AtomicStoreRelaxed(&GInitialized, true);
}

////////////////////////////////////////////////////////////////////////////////
static void Writer_InternalShutdown()
{
	if (!AtomicLoadRelaxed(&GInitialized))
	{
		return;
	}

	Writer_WorkerJoin();
	Writer_Close();

	Writer_ShutdownControl();
	Writer_ShutdownPool();
#if TRACE_PRIVATE_ALLOW_IMPORTANTS
	Writer_ShutdownSharedBuffers();
	Writer_ShutdownCache();
#endif
	Writer_ShutdownTail();

#if TRACE_PRIVATE_BUFFER_SEND
	if (GSendBuffer)
	{
		Writer_MemoryFree(GSendBuffer, GSendBufferSize);
#if TRACE_PRIVATE_STATISTICS
		AtomicSubRelaxed(&GTraceStatistics.FixedBufferAllocated, uint32(GSendBufferSize));
#endif
		GSendBuffer = nullptr;
		GSendBufferCursor = nullptr;
	}
#endif

	AtomicStoreRelaxed(&GInitialized, false);
}

////////////////////////////////////////////////////////////////////////////////
void Writer_InternalInitialize()
{
	using namespace Private;

	if (AtomicLoadRelaxed(&GInitialized))
	{
		return;
	}

	static struct FInitializer
	{
		FInitializer()
		{
			Writer_InternalInitializeImpl();
		}
		~FInitializer()
		{
			/* We'll not shut anything down here so we can hopefully capture
			 * any subsequent events. However, we will shutdown the worker
			 * thread and leave it for something else to call update() (mem
			 * tracing at time of writing). Windows will have already done
			 * this implicitly in ExitProcess() anyway. */
			Writer_WorkerJoin();
		}
	} Initializer;
}

////////////////////////////////////////////////////////////////////////////////
void Writer_Initialize(const FInitializeDesc& Desc)
{
	// Store scope callbacks for optional profiling
	FProfilerScope::OnScopeBegin = Desc.OnScopeBeginFunc;
	FProfilerScope::OnScopeEnd = Desc.OnScopeEndFunc;

	FProfilerScope _(__func__);
	GBlockPoolMaxSize = Desc.BlockPoolMaxSize;

	Writer_InitializeTail(Desc.TailSizeBytes);

#if TRACE_PRIVATE_ALLOW_IMPORTANTS
	if (Desc.bUseImportantCache)
	{
		Writer_InitializeCache();
	}
#endif

	if (Desc.ThreadSleepTimeInMS != 0)
	{
		GSleepTimeInMS = Desc.ThreadSleepTimeInMS;
	}

	if (Desc.bUseWorkerThread)
	{
		Writer_WorkerCreate();
	}

	// Store the session guid if specified, otherwise generate one
	if (!Desc.SessionGuid[0] & !Desc.SessionGuid[1] & !Desc.SessionGuid[2] & !Desc.SessionGuid[3])
	{
		Writer_CreateGuid(&GSessionGuid);
	}
	else
	{
		memcpy(&GSessionGuid, &Desc.SessionGuid, sizeof(FTraceGuid));
	}

	// Store callback on connection
	GOnConnection = Desc.OnConnectionFunc;
	AtomicStoreRelaxed(&GOnUpdate, Desc.OnUpdateFunc);
	GOnClose = Desc.OnCloseFunc;

	// Allow the worker thread to start updating
	AtomicStoreRelease(&GUpdateInProgress, uint32(0));
}

////////////////////////////////////////////////////////////////////////////////
void Writer_Shutdown()
{
	Writer_InternalShutdown();
}

////////////////////////////////////////////////////////////////////////////////
void Writer_Update()
{
	if (!GWorkerThread)
	{
		Writer_WorkerUpdate();
	}
}

////////////////////////////////////////////////////////////////////////////////
UPTRINT Writer_PackSendFlags(UPTRINT DataHandle, uint32 Flags)
{
	// Passing ownership of IO to the worker thread via a single 64 bit value is
	// convenient and saves a lot of machinery for something that mostly never
	// happens, or perhaps just once or twice. Here we make the assumption that
	// our supported platforms' handles are low integer file descriptor IDs or
	// addresses and thus we have some most-significant bits to use for flags.

	// Guard against the assumption being wrong.
	if (DataHandle & 0xffff'0000'0000'0000ull)
	{
		GWriterState.Close(DataHandle);
		return 0;
	}

	return DataHandle | (UPTRINT(Flags) << 48ull);
}

////////////////////////////////////////////////////////////////////////////////
bool Writer_SetPendingHandle(UPTRINT NewHandle, IoWriteFunc Write, IoCloseFunc Close, IoUpdateFunc Update = nullptr, IoIsReadyFunc IsReady = nullptr)
{
	FLock Lock(&GPendingWriterLock);
	if (AtomicLoadAcquire(&GPendingDataHandle))
	{
		// Some other thread has already set the pending handle
		return false;
	}
	// Write the pending writer state first, then release the
	// pending data handle to guarantee the state is available.
	GPendingWriterState = {Write, Close, Update, IsReady};
	AtomicStoreRelease(&GPendingDataHandle, NewHandle);
	return true;
}

////////////////////////////////////////////////////////////////////////////////
void Writer_ClearPendingHandle()
{
	FLock Lock(&GPendingWriterLock);
	GPendingWriterState = {nullptr, nullptr, nullptr, nullptr};
	AtomicStoreRelease(&GPendingDataHandle, UPTRINT(0));
}

////////////////////////////////////////////////////////////////////////////////
bool Writer_SendTo(const ANSICHAR* Host, uint32 Flags, uint32 Port)
{
#if TRACE_PRIVATE_ALLOW_TCP
	if (AtomicLoadRelaxed(&GPendingDataHandle))
	{
		return false;
	}

	Writer_InternalInitialize();

	Port = Port ? Port : 1981;
	UPTRINT DataHandle = TcpSocketConnect(Host, uint16(Port));
	if (!DataHandle)
	{
		UE_TRACE_ERRORMESSAGE_F(ConnectError, GetLastErrorCode(), "Connecting to host (%s:%u)", Host, Port);
		return false;
	}

	DataHandle = Writer_PackSendFlags(DataHandle, Flags);
	if (!DataHandle)
	{
		UE_TRACE_MESSAGE(ConnectError, "Handle was unexpectedly using MSB flags.");
		return false;
	}

	return Writer_SetPendingHandle(DataHandle, &IoWrite, &IoClose);
#else
	return false;
#endif
}

////////////////////////////////////////////////////////////////////////////////
bool Writer_WriteTo(const ANSICHAR* Path, uint32 Flags)
{
#if TRACE_PRIVATE_ALLOW_FILE
	if (AtomicLoadRelaxed(&GPendingDataHandle))
	{
		return false;
	}

	Writer_InternalInitialize();

	UPTRINT DataHandle = FileOpen(Path);
	if (!DataHandle)
	{
		UE_TRACE_ERRORMESSAGE_F(FileOpenError, GetLastErrorCode(), "Opening file (%s)", Path);
		return false;
	}

	DataHandle = Writer_PackSendFlags(DataHandle, Flags);
	if (!DataHandle)
	{
		return false;
	}

	return Writer_SetPendingHandle(DataHandle, &IoWrite, &IoClose);
#else
	return false;
#endif
}

////////////////////////////////////////////////////////////////////////////////
bool Writer_RelayTo(UPTRINT InHandle, IoWriteFunc WriteFunc, IoCloseFunc CloseFunc, uint16 Flags)
{
	if (AtomicLoadRelaxed(&GPendingDataHandle))
	{
		return false;
	}

	Writer_InternalInitialize();

	UPTRINT DataHandle = InHandle;
	DataHandle = Writer_PackSendFlags(DataHandle, Flags);
	if (!DataHandle)
	{
		return false;
	}

	return Writer_SetPendingHandle(DataHandle, WriteFunc, CloseFunc);
}

////////////////////////////////////////////////////////////////////////////////
struct WorkerUpdateLock
{
	WorkerUpdateLock()
	{
		CyclesPerSecond = TimeGetFrequency();
		StartSeconds = GetTime();

		while (!AtomicCompareExchangeAcquire(&GUpdateInProgress, 1u, 0u))
		{
			ThreadSleep(0);

			if (TimedOut())
			{
				break;
			}
		}
	}

	~WorkerUpdateLock()
	{
		AtomicExchangeRelease(&GUpdateInProgress, 0u);
	}

	double GetTime()
	{
		return static_cast<double>(TimeGetTimestamp()) / static_cast<double>(CyclesPerSecond);
	}

	bool TimedOut()
	{
		const double WaitTime = GetTime() - StartSeconds;
		return WaitTime > MaxWaitSeconds;
	}

	uint64 CyclesPerSecond;
	double StartSeconds;
	inline const static double MaxWaitSeconds = 1.0;
};

////////////////////////////////////////////////////////////////////////////////
template <typename Type>
struct TStashGlobal
{
	TStashGlobal(Type& Global)
		: Variable(Global)
		, Stashed(Global)
	{
		Variable = {};
	}

	TStashGlobal(Type& Global, const Type& Value)
		: Variable(Global)
		, Stashed(Global)
	{
		Variable = Value;
	}

	~TStashGlobal()
	{
		Variable = Stashed;
	}

private:
	Type& Variable;
	Type Stashed;
};

////////////////////////////////////////////////////////////////////////////////
struct FSnapshotTarget
{
	enum EType { FileTarget, HostTarget };
	EType Type;
	uint32 MaxTailSize;
	union
	{
		 struct
		 {
			 const ANSICHAR* Path;
		 } File;
		 struct
		 {
			 const ANSICHAR* Host;
			 uint32 Port;
		 } Host;
	};
};

////////////////////////////////////////////////////////////////////////////////
bool Writer_WriteSnapshot(const FSnapshotTarget& Target)
{
	if (!Writer_IsTailing())
	{
		return false;
	}

	WorkerUpdateLock UpdateLock;

	// We have a timeout just in case the worker thread goes off the rails
	//  We are called by diagnostic handlers like crash reporter, do not deadlock
	if (UpdateLock.TimedOut())
	{
		return false;
	}

	// Bring everything up to date with the active tracing connection.
	//  Any connection writes after we call the worker update
	//  will need to treat source data structures as read-only.
	//  Those structures can be stateful about what has or has not been
	//  written (cursor state, "new" event, etc) to the tracing connection.
	//  We are pre-empting the connection to write the snapshot.
	//
	// Doing a full pump of the data here is more robust than opening
	//  pathways through each data structure for immutable writes because
	//  the data structures are order-dependent and inter-referential.
	//  Not writing all state can very easily lead to bugs in either the
	//  snapshot or, even worse, the tracing connection. Parsers only
	//  have a limited tolerance to gaps/out-of-order event packets.
	Writer_WorkerUpdateInternal();

	// Force flush the send buffer so that platforms that use internal send buffers
	// don't loose data.
	Writer_FlushSendBuffer();

	{
		const bool bExistingDataHandle = GDataHandle != 0;

		TStashGlobal DataHandle(GDataHandle);
		TStashGlobal StateWrite(GWriterState.Write);
		TStashGlobal StateClose(GWriterState.Close);
		TStashGlobal StateUpdate(GWriterState.Update);
		TStashGlobal StateIsReady(GWriterState.IsReady);
		TStashGlobal SyncPacketCountdown(GSyncPacketCountdown, GNumSyncPackets);
		TStashGlobal TraceStatistics(GTraceStatistics);

		GWriterState = { &IoWrite, &IoClose, nullptr, nullptr };

		if (Target.Type == FSnapshotTarget::EType::FileTarget)
		{
#if TRACE_PRIVATE_ALLOW_FILE
			// Open the snapshot file
			GDataHandle = FileOpen(Target.File.Path);
#endif
		}
		else
		{
#if TRACE_PRIVATE_ALLOW_TCP
			// Open the snapshot connection and write
			const uint32 Port = Target.Host.Port ? Target.Host.Port : 1981;
			GDataHandle = TcpSocketConnect(Target.Host.Host, uint16(Port));
			if (!GDataHandle)
			{
				return false;
			}
			GDataHandle = Writer_PackSendFlags(GDataHandle, 0);
#endif
		}

		// Write the file header
		if (!GDataHandle || !Writer_SessionPrologue())
		{
			UE_TRACE_ERRORMESSAGE(FileOpenError, GetLastErrorCode());
			if (bExistingDataHandle)
			{
				UE_TRACE_MESSAGE(Display, "Creating a snapshot during ongoing trace "
					"is known to fail on some combinations of platforms and hardware.");
			}
			return false;
		}

		// The first events we will send are ones that describe the trace's events
		Writer_DescribeEvents(FEventNode::Read());

		// Send cached events (i.e. importants)
#if TRACE_PRIVATE_ALLOW_IMPORTANTS
		Writer_CacheOnConnect();
#endif

		// Issue on connection callback. This allows writing events that are
		// not cached but important for the cache
		Writer_CallbackOnConnect();

		// Finally write the events in the tail buffer
		Writer_TailOnConnect(Target.MaxTailSize);

		// Send sync packets to help parsers digest any out-of-order events
		GSyncPacketCountdown = GNumSyncPackets;
		while (GSyncPacketCountdown > 0)
		{
			Writer_SendSync();
		}

		Writer_Close();
	}

	return true;
}

////////////////////////////////////////////////////////////////////////////////
bool Writer_WriteSnapshotTo(const ANSICHAR* Path, uint32 MaxTailSize)
{
	FSnapshotTarget Target;
	Target.Type = FSnapshotTarget::EType::FileTarget;
	Target.MaxTailSize = MaxTailSize;
	Target.File.Path = Path;
	return Writer_WriteSnapshot(Target);
}

////////////////////////////////////////////////////////////////////////////////
bool Writer_SendSnapshotTo(const ANSICHAR* Host, uint32 Port, uint32 MaxTailSize)
{
	FSnapshotTarget Target;
	Target.Type = FSnapshotTarget::EType::HostTarget;
	Target.MaxTailSize = MaxTailSize;
	Target.Host.Host = Host;
	Target.Host.Port = Port;
	return Writer_WriteSnapshot(Target);
}

////////////////////////////////////////////////////////////////////////////////
bool Writer_IsTracing()
{
	return AtomicLoadRelaxed(&GDataHandle) != 0 || AtomicLoadRelaxed(&GPendingDataHandle) != 0;
}

////////////////////////////////////////////////////////////////////////////////
bool Writer_IsTracingTo(uint32 (&OutSessionGuid)[4], uint32 (&OutTraceGuid)[4])
{
	if (Writer_IsTracing())
	{
		memcpy(&OutSessionGuid, &GSessionGuid, sizeof(OutSessionGuid));
		memcpy(&OutTraceGuid, &GTraceGuid, sizeof(OutTraceGuid));
		return true;
	}
	return false;
}

////////////////////////////////////////////////////////////////////////////////
bool Writer_Stop()
{
	if (GPendingDataHandle || !GDataHandle)
	{
		return false;
	}

	AtomicStoreRelaxed(&GPendingDataHandle, ~UPTRINT(0));
	return true;
}

////////////////////////////////////////////////////////////////////////////////
void Writer_Panic()
{
	AtomicStoreRelease(&GWorkerThreadQuit, true);
	if (GWorkerThread && GWorkerThread != Writer_GetThreadId())
	{
		ThreadDestroy(GWorkerThread);
	}
	// Set the update in progress to make sure updates cannot be executed
	// and snapshots cannot be written.
	AtomicStoreRelease(&GUpdateInProgress, 1u);
	GWorkerThread = 0;
	UE_TRACE_MESSAGE(PanicError, "Trace has encountered a catastrophic failure. Tracing will be unavailable for the rest of the process lifetime.");
}

} // namespace Private
} // namespace Trace
} // namespace UE

#endif // TRACE_PRIVATE_MINIMAL_ENABLED
