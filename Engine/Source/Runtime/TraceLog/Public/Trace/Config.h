// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "HAL/Platform.h"

/**
 * Public control for enabling tracing.
 *
 * This define controls all the TraceLog functionality and is
 * used under normal circumstances.
 */
#if !defined(UE_TRACE_ENABLED)
#	if !UE_BUILD_SHIPPING && !IS_PROGRAM
#		if PLATFORM_WINDOWS || PLATFORM_UNIX || PLATFORM_APPLE || PLATFORM_ANDROID
#			define UE_TRACE_ENABLED	1
#		endif
#	endif
#endif

#if !defined(UE_TRACE_ENABLED)
#	define UE_TRACE_ENABLED 0
#endif

/**
 * EXPERIMENTAL!: Optional control to enable tracing in shipping configuration.
 *
 * When this define is enabled TraceLog functionality can be enabled in
 * shipping builds. Note that regular tracing system that relies on UE_TRACE_ENABLED
 * will not be active. This is intentional in order to avoid unintentional functionality
 * slipping into shipping builds.
 */
#if !defined(UE_TRACE_ENABLED_SHIPPING_EXPERIMENTAL)
#	define UE_TRACE_ENABLED_SHIPPING_EXPERIMENTAL 0
#endif


/**
 * Public control for the minimal set of tracing.
 */
#if !defined(UE_TRACE_MINIMAL_ENABLED)
#	define UE_TRACE_MINIMAL_ENABLED UE_TRACE_ENABLED
#endif

/**
 * Internal defined used inside this library. Do not set outside of TraceLog.
 */
#if UE_TRACE_ENABLED || UE_TRACE_MINIMAL_ENABLED
#	define TRACE_PRIVATE_MINIMAL_ENABLED 1
#else
#	define TRACE_PRIVATE_MINIMAL_ENABLED 0
#endif

#if UE_TRACE_ENABLED
#	define TRACE_PRIVATE_FULL_ENABLED 1
#else
#	define TRACE_PRIVATE_FULL_ENABLED 0
#endif

/**
 * Shipping trace is a subset of regular tracing. If regular trace is
 * enabled, shipping trace must also be enabled.
 */
static_assert(TRACE_PRIVATE_FULL_ENABLED == 0 || TRACE_PRIVATE_MINIMAL_ENABLED != 0, "Minimal is a subset of full tracing."); //-V547
#if UE_BUILD_SHIPPING && !IS_PROGRAM && !defined(UE_TRACE_FULL_SHIPPING_ENABLED_UNSUPPORTED)
static_assert(!TRACE_PRIVATE_FULL_ENABLED, "Full tracing in shipping is not supported.");
static_assert(UE_TRACE_MINIMAL_ENABLED == 0 || UE_TRACE_ENABLED_SHIPPING_EXPERIMENTAL, "Minimal trace in shipping is currently experimental");
#endif


/**
 * Current protocol version.
 */
#if TRACE_PRIVATE_MINIMAL_ENABLED
#	define TRACE_PRIVATE_PROTOCOL_7
#endif

/**
 * Control the socket control component. By default we disable it if only
 * shipping trace is enabled.
 */
#if !defined(UE_TRACE_ALLOW_TCP_CONTROL)
#	define TRACE_PRIVATE_ALLOW_TCP_CONTROL TRACE_PRIVATE_FULL_ENABLED
#else
#	define TRACE_PRIVATE_ALLOW_TCP_CONTROL UE_TRACE_ALLOW_TCP_CONTROL
#endif

/**
 * Control tracing to tcp socket connections.
 */
#if !defined(UE_TRACE_ALLOW_TCP_TRACING)
#	define TRACE_PRIVATE_ALLOW_TCP TRACE_PRIVATE_FULL_ENABLED
#else
#	define TRACE_PRIVATE_ALLOW_TCP UE_TRACE_ALLOW_TCP_TRACING
#endif

/**
 * Control tracing to files.
 */
#if !defined(UE_TRACE_ALLOW_FILE_TRACING)
#	define TRACE_PRIVATE_ALLOW_FILE TRACE_PRIVATE_FULL_ENABLED
#else
#	define TRACE_PRIVATE_ALLOW_FILE UE_TRACE_ALLOW_FILE_TRACING
#endif

/**
 * Control tracing to secure socket connections.
 */
#if !defined(UE_TRACE_ALLOW_SECURE_TRACING)
#	define UE_TRACE_ALLOW_SECURE_TRACING 0
#	define TRACE_PRIVATE_ALLOW_SECURE_TRACING 0
#else
#	define TRACE_PRIVATE_ALLOW_SECURE_TRACING UE_TRACE_ALLOW_SECURE_TRACING
#endif

/**
 * Control if important events are enabled.
 */
#if !defined(UE_TRACE_ALLOW_IMPORTANT_EVENTS)
#	define TRACE_PRIVATE_ALLOW_IMPORTANTS TRACE_PRIVATE_FULL_ENABLED
#else
#	define TRACE_PRIVATE_ALLOW_IMPORTANTS UE_TRACE_ALLOW_IMPORTANT_EVENTS
#endif

/**
 * Default Tail buffer size in bytes. Defaults to 0 when Minimal Tracing.
 */
#if !defined(UE_TRACE_DEFAULT_TAIL_SIZE)
#   if TRACE_PRIVATE_FULL_ENABLED
#       define UE_TRACE_DEFAULT_TAIL_SIZE 4 << 20
#   else
#       define UE_TRACE_DEFAULT_TAIL_SIZE 0
#   endif
#endif

/**
 * Default block pool size in bytes. See BlockPool implementation for discussion about
 * overriding this value.
 */
#if !defined(UE_TRACE_BLOCK_POOL_MAXSIZE)
#	define UE_TRACE_BLOCK_POOL_MAXSIZE 79 << 20
#endif

/**
* Default size of each block in the block pool.
*/
#if !defined(UE_TRACE_BLOCK_SIZE)
#	define UE_TRACE_BLOCK_SIZE (4 << 10)
#endif

/**
* Time for writer thread to sleep in between writes.
*/
#if !defined(UE_TRACE_WRITER_SLEEP_MS)
#	define UE_TRACE_WRITER_SLEEP_MS (17)
#endif

/**
 * Enable packet verification. Only useful when looking for transmission bugs. Note that in order
 * to avoid making a new protocol version, enabling this makes existing version 7 traces incompatible.
 */
#if !defined(UE_TRACE_PACKET_VERIFICATION)
#	define UE_TRACE_PACKET_VERIFICATION 0
#endif

/**
 * Enable tracking of memory and bandwidth statistics.
 */
#if !defined(UE_TRACE_STATISTICS_ENABLED)
#	define UE_TRACE_STATISTICS_ENABLED (!(UE_BUILD_SHIPPING || USING_THREAD_SANITISER)) 
#endif

/**
 * Enable built in events to send statistics. Enabled by default
 * when statistics tracking is enabled.
 */
#if !defined(UE_TRACE_EMIT_STATISTICS)
#	define UE_TRACE_EMIT_STATISTICS UE_TRACE_STATISTICS_ENABLED
#endif

/**
 * Assert functionality. When integrating TraceLog define this to let it use 
 * built in assert functionality. The format string is specified using wide char.
 *
 * @note: Asserts must only be used in inlined code due linking.
 * @note: See UE checkf for expected format
 * @note: In UE with non-monolithic targets, TraceLog module cannot use asserts. So
 * we use the suppress macro to reset any global defines.
 */
#if defined(UE_TRACE_SUPPRESS_ASSERT)
#	undef UE_TRACE_ASSERT_FMT
#endif
#if !defined(UE_TRACE_ASSERT_FMT)
#	define	UE_TRACE_ASSERT_FMT(Cond, Fmt, ...)
#endif
