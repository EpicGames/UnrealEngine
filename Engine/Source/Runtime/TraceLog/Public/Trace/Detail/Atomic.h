// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include <atomic>

#if PLATFORM_CPU_X86_FAMILY
#	include <immintrin.h>
#endif

#if !defined(TRACE_PRIVATE_THREAD_YIELD)
#	define TRACE_PRIVATE_THREAD_YIELD 0
#endif

namespace UE {
namespace Trace {
namespace Private {

////////////////////////////////////////////////////////////////////////////////
template <typename Type> Type	AtomicLoadRelaxed(Type volatile* Source);
template <typename Type> Type	AtomicLoadAcquire(Type volatile* Source);
template <typename Type> void	AtomicStoreRelaxed(Type volatile* Target, Type Value);
template <typename Type> void	AtomicStoreRelease(Type volatile* Target, Type Value);
template <typename Type> Type	AtomicExchangeAcquire(Type volatile* Target, Type Value);
template <typename Type> Type	AtomicExchangeRelease(Type volatile* Target, Type Value);
template <typename Type> bool	AtomicCompareExchangeRelaxed(Type volatile* Target, Type New, Type Expected);
template <typename Type> bool	AtomicCompareExchangeAcquire(Type volatile* Target, Type New, Type Expected);
template <typename Type> bool	AtomicCompareExchangeRelease(Type volatile* Target, Type New, Type Expected);
template <typename Type> Type	AtomicAddRelaxed(Type volatile* Target, Type Value);
template <typename Type> Type	AtomicAddRelease(Type volatile* Target, Type Value);
template <typename Type> Type	AtomicAddAcquire(Type volatile* Target, Type Value);
template <typename Type> Type	AtomicSubRelaxed(Type volatile* Target, Type Value);
template <typename Type> Type	AtomicSubRelease(Type volatile* Target, Type Value);
template <typename Type> Type	AtomicSubAcquire(Type volatile* Target, Type Value);
void							PlatformYield();

////////////////////////////////////////////////////////////////////////////////
inline void PlatformYield()
{
#if TRACE_PRIVATE_THREAD_YIELD
	extern void ThreadYield();
	ThreadYield();
#elif PLATFORM_USE_SSE2_FOR_THREAD_YIELD
	_mm_pause();
#elif PLATFORM_CPU_ARM_FAMILY
#	if defined(_MSC_VER) && !defined(__clang__) // MSVC
		__yield();
#	else
		__builtin_arm_yield();
#	endif
#else
	#error Unsupported architecture!
#endif
}

////////////////////////////////////////////////////////////////////////////////
template <typename Type>
inline Type AtomicLoadRelaxed(Type volatile* Source)
{
	std::atomic<Type>* T = (std::atomic<Type>*) Source;
	return T->load(std::memory_order_relaxed);
}

////////////////////////////////////////////////////////////////////////////////
template <typename Type>
inline Type AtomicLoadAcquire(Type volatile* Source)
{
	std::atomic<Type>* T = (std::atomic<Type>*) Source;
	return T->load(std::memory_order_acquire);
}

////////////////////////////////////////////////////////////////////////////////
template <typename Type>
inline void AtomicStoreRelaxed(Type volatile* Target, Type Value)
{
	std::atomic<Type>* T = (std::atomic<Type>*) Target;
	T->store(Value, std::memory_order_relaxed);
}

////////////////////////////////////////////////////////////////////////////////
template <typename Type>
inline void AtomicStoreRelease(Type volatile* Target, Type Value)
{
	std::atomic<Type>* T = (std::atomic<Type>*) Target;
	T->store(Value, std::memory_order_release);
}

////////////////////////////////////////////////////////////////////////////////
template <typename Type>
inline Type AtomicExchangeAcquire(Type volatile* Target, Type Value)
{
	std::atomic<Type>* T = (std::atomic<Type>*) Target;
	return T->exchange(Value, std::memory_order_acquire);
}

////////////////////////////////////////////////////////////////////////////////
template <typename Type>
inline Type AtomicExchangeRelease(Type volatile* Target, Type Value)
{
	std::atomic<Type>* T = (std::atomic<Type>*) Target;
	return T->exchange(Value, std::memory_order_release);
}

////////////////////////////////////////////////////////////////////////////////
template <typename Type>
inline bool AtomicCompareExchangeRelaxed(Type volatile* Target, Type New, Type Expected)
{
	std::atomic<Type>* T = (std::atomic<Type>*) Target;
	return T->compare_exchange_weak(Expected, New, std::memory_order_relaxed);
}

////////////////////////////////////////////////////////////////////////////////
template <typename Type>
inline bool AtomicCompareExchangeAcquire(Type volatile* Target, Type New, Type Expected)
{
	std::atomic<Type>* T = (std::atomic<Type>*) Target;
	return T->compare_exchange_weak(Expected, New, std::memory_order_acquire);
}

////////////////////////////////////////////////////////////////////////////////
template <typename Type>
inline bool AtomicCompareExchangeRelease(Type volatile* Target, Type New, Type Expected)
{
	std::atomic<Type>* T = (std::atomic<Type>*) Target;
	return T->compare_exchange_weak(Expected, New, std::memory_order_release);
}

////////////////////////////////////////////////////////////////////////////////
template <typename Type>
inline Type AtomicAddRelaxed(Type volatile* Target, Type Value)
{
	std::atomic<Type>* T = (std::atomic<Type>*) Target;
	return T->fetch_add(Value, std::memory_order_relaxed);
}

////////////////////////////////////////////////////////////////////////////////
template <typename Type>
inline Type AtomicAddAcquire(Type volatile* Target, Type Value)
{
	std::atomic<Type>* T = (std::atomic<Type>*) Target;
	return T->fetch_add(Value, std::memory_order_acquire);
}

////////////////////////////////////////////////////////////////////////////////
template <typename Type>
inline Type AtomicAddRelease(Type volatile* Target, Type Value)
{
	std::atomic<Type>* T = (std::atomic<Type>*) Target;
	return T->fetch_add(Value, std::memory_order_release);
}

////////////////////////////////////////////////////////////////////////////////
template <typename Type>
inline Type AtomicSubRelaxed(Type volatile* Target, Type Value)
{
	std::atomic<Type>* T = (std::atomic<Type>*) Target;
	return T->fetch_sub(Value, std::memory_order_relaxed);
}

////////////////////////////////////////////////////////////////////////////////
template <typename Type>
inline Type AtomicSubAcquire(Type volatile* Target, Type Value)
{
	std::atomic<Type>* T = (std::atomic<Type>*) Target;
	return T->fetch_sub(Value, std::memory_order_acquire);
}

////////////////////////////////////////////////////////////////////////////////
template <typename Type>
inline Type AtomicSubRelease(Type volatile* Target, Type Value)
{
	std::atomic<Type>* T = (std::atomic<Type>*) Target;
	return T->fetch_sub(Value, std::memory_order_release);
}
	
} // namespace Private
} // namespace Trace
} // namespace UE
