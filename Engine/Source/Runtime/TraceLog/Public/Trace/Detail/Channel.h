// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Trace/Config.h"
#include "Trace/Trace.h"

#if TRACE_PRIVATE_MINIMAL_ENABLED

#include "CoreTypes.h"

namespace UE {
namespace Trace {


typedef bool ChannelIterCallback(const FChannelInfo& OutChannelInfo, void*);

/*
	A named channel which can be used to filter trace events. Channels can be
	combined using the '|' operator which allows expressions like

	```
	UE_TRACE_LOG(FooWriter, FooEvent, FooChannel|BarChannel);
	```

	Note that this works as an AND operator, similar to how a bitmask is constructed.

	Channels are by default enabled until FChannel::Initialize() is called. This is to allow
	events to be emitted during static initialization. In fact all events during
	this phase are always emitted.
*/
class FChannel
{
public:
	struct Iter
	{
		~Iter();
		const FChannel*	GetNext();
		void*Inner[3];
	};

	struct InitArgs
	{
		/**
		 * User facing description string.
		 */
		const ANSICHAR* Desc;
		/**
		 * If set, channel cannot be changed during a run, only set through command line.
		 */
		bool bReadOnly;
	};

	/**
	 * Initializes a channel. Used by channel macros, do no use directly.
	 */
	TRACELOG_API void Setup(const ANSICHAR* InChannelName, const InitArgs& Args);

	/**
	 * Allows channels to act pre state change deny the change from happening.
	 *
	 * @param bNewState Desired new state for channel
	 * @param OutReason Deny reason. Set only when returning false.
	 * @return True when new state is acceptable, false otherwise.
	 */
	virtual bool OnToggle(bool bNewState, const TCHAR** OutReason) { return true; }

	/**
	 * Allows channels to act post state change. At this point it is possible to emit
	 * events on this channel.
	 *
	 * @param bNewState New state for the channel
	 */
	virtual void OnToggled(bool bNewState) {}

	/**
	 * Toggles the channel state (on/off). If the channel could not be toggled an optional
	 * deny reason can be accessed.
	 *
	 * @param bEnabled Requested state
	 * @param OutReason Optional string pointer. Will contain deny reason on failure.
	 * @return New channel state
	 */
	TRACELOG_API bool	Toggle(bool bEnabled, const TCHAR** OutReason = nullptr);

	static void Initialize();
	static Iter	ReadNew();
	void Announce() const;
	static bool Toggle(const ANSICHAR* ChannelName, bool bEnabled);
	static void ToggleAll(bool bEnabled);
	static void PanicDisableAll(); // Disabled channels wont be logged with UE_TRACE_LOG
	static FChannel* FindChannel(const ANSICHAR* ChannelName);
	static FChannel* FindChannel(FChannelId ChannelId);
	static void EnumerateChannels(ChannelIterCallback Func, void* User);
	bool IsEnabled() const;
	bool IsReadOnly() const { return Args.bReadOnly; };
	uint32 GetName(const ANSICHAR** OutName) const;
	explicit operator bool () const;
	bool operator | (const FChannel& Rhs) const;

private:
	bool ToggleInternal(bool bEnabled, bool bIssueCallback, const TCHAR** OutReason = nullptr);
	static void ToggleAllInternal(bool bEnabled, bool bIssueCallback);
	
	FChannel* volatile Next;
	struct
	{
		const ANSICHAR* Ptr;
		uint32 Len;
		uint32 Hash;
	} Name;
	volatile int32 Enabled;
	InitArgs Args;
	volatile uint8 Lock;
};

inline uint32 FChannel::GetName(const ANSICHAR** OutName) const
{
	if (OutName != nullptr)
	{
		*OutName = Name.Ptr;
		return Name.Len;
	}
	return 0;
}


} // namespace Trace
} // namespace UE

#else

// Since we use this type in macros we need
// provide an empty definition when trace is
// not enabled.
namespace UE::Trace { class FChannel {}; }

#endif // TRACE_PRIVATE_MINIMAL_ENABLED
