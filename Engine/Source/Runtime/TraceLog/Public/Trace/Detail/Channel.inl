// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channel.h"
#include "Trace/Detail/Atomic.h"

#if TRACE_PRIVATE_MINIMAL_ENABLED

namespace UE {
namespace Trace {

extern TRACELOG_API FChannel& TraceLogChannel;

////////////////////////////////////////////////////////////////////////////////
inline bool FChannel::IsEnabled() const
{
	return Private::AtomicLoadRelaxed((volatile int32*) &Enabled) >= 0;
}

////////////////////////////////////////////////////////////////////////////////
inline FChannel::operator bool () const
{
	return IsEnabled();
}

////////////////////////////////////////////////////////////////////////////////
inline bool FChannel::operator | (const FChannel& Rhs) const
{
	return IsEnabled() && Rhs.IsEnabled();
}

} // namespace Trace
} // namespace UE

#endif // TRACE_PRIVATE_MINIMAL_ENABLED
