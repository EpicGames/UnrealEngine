// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Trace/Config.h"

#if TRACE_PRIVATE_MINIMAL_ENABLED

#include "Field.h"

namespace UE {
namespace Trace {
namespace Private {

////////////////////////////////////////////////////////////////////////////////
struct FEventInfo
{
	enum
	{
		Flag_None			= 0,
		Flag_Important		= 1 << 0,
		Flag_MaybeHasAux	= 1 << 1,
		Flag_NoSync			= 1 << 2,
		Flag_Definition8	= 1 << 3,
		Flag_Definition16	= 1 << 4,
		Flag_Definition32	= 1 << 5,
		Flag_Definition64	= 1 << 6,
		Flag_MaybeImportant	= 1 << 7,

		DefinitionBits		= Flag_Definition8 | Flag_Definition16 | Flag_Definition32 | Flag_Definition64,
	};

	FLiteralName			LoggerName;
	FLiteralName			EventName;
	const FFieldDesc*		Fields;
	uint16					FieldCount;
	uint16					Flags;
};

////////////////////////////////////////////////////////////////////////////////
class FEventNode
{
public:
	struct FIter
	{
		const FEventNode*	GetNext();
		void*				Inner;
	};

	static FIter			Read();
	static FIter			ReadNew();
	static void				OnConnect();
	TRACELOG_API uint32		Initialize(const FEventInfo* InInfo);
	void					Describe() const;
	uint32					GetUid() const { return Uid; }

private:
	FEventNode*				Next;
	const FEventInfo*		Info;
	uint32					Uid;
};

} // namespace Private
} // namespace Trace
} // namespace UE

#else

// Since we use this type in macros we need
// provide an empty definition when trace is
// not enabled.
namespace UE::Trace::Private { struct FEventInfo {}; }

#endif // TRACE_PRIVATE_MINIMAL_ENABLED
