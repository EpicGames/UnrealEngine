// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Trace/Config.h"

#if TRACE_PRIVATE_MINIMAL_ENABLED

#include "Atomic.h"
#include "Protocol.h"
#include "Templates/UnrealTemplate.h"
#include "Trace/Trace.h"
#include "Writer.inl"

/* Statically sized fields (e.g. UE_TRACE_EVENT_FIELD(float[4], Colours)) are
 * not supported as yet. No call for them. The following define is used to track
 * where and partially how to implement them */
#define STATICALLY_SIZED_ARRAY_FIELDS_SUPPORT 0

namespace UE {
namespace Trace {

namespace Private
{

////////////////////////////////////////////////////////////////////////////////
UE_TRACE_API void Field_WriteAuxData(uint32, const uint8*, int32);
UE_TRACE_API void Field_WriteStringAnsi(uint32, const ANSICHAR*, int32);
UE_TRACE_API void Field_WriteStringAnsi(uint32, const WIDECHAR*, int32);
UE_TRACE_API void Field_WriteStringWide(uint32, const WIDECHAR*, int32);

class FEventNode;

} // namespace Private

////////////////////////////////////////////////////////////////////////////////
enum DisabledField {};

template <typename Type> struct TFieldType;

template <> struct TFieldType<DisabledField>{ enum { Tid = 0,						Size = 0 }; };
template <> struct TFieldType<bool>			{ enum { Tid = int(EFieldType::Bool),	Size = sizeof(bool) }; };
template <> struct TFieldType<int8>			{ enum { Tid = int(EFieldType::Int8),	Size = sizeof(int8) }; };
template <> struct TFieldType<int16>		{ enum { Tid = int(EFieldType::Int16),	Size = sizeof(int16) }; };
template <> struct TFieldType<int32>		{ enum { Tid = int(EFieldType::Int32),	Size = sizeof(int32) }; };
template <> struct TFieldType<int64>		{ enum { Tid = int(EFieldType::Int64),	Size = sizeof(int64) }; };
template <> struct TFieldType<uint8>		{ enum { Tid = int(EFieldType::Uint8),	Size = sizeof(uint8) }; };
template <> struct TFieldType<uint16>		{ enum { Tid = int(EFieldType::Uint16),	Size = sizeof(uint16) }; };
template <> struct TFieldType<uint32>		{ enum { Tid = int(EFieldType::Uint32),	Size = sizeof(uint32) }; };
template <> struct TFieldType<uint64>		{ enum { Tid = int(EFieldType::Uint64),	Size = sizeof(uint64) }; };
template <> struct TFieldType<float>		{ enum { Tid = int(EFieldType::Float32),Size = sizeof(float) }; };
template <> struct TFieldType<double>		{ enum { Tid = int(EFieldType::Float64),Size = sizeof(double) }; };
template <class T> struct TFieldType<T*>	{ enum { Tid = int(EFieldType::Pointer),Size = sizeof(void*) }; };

template <typename T>
struct TFieldType<T[]>
{
	enum
	{
		Tid  = int(TFieldType<T>::Tid)|int(EFieldType::Array),
		Size = 0,
	};
};

#if STATICALLY_SIZED_ARRAY_FIELDS_SUPPORT
template <typename T, int N>
struct TFieldType<T[N]>
{
	enum
	{
		Tid  = int(TFieldType<T>::Tid)|int(EFieldType::Array),
		Size = sizeof(T[N]),
	};
};
#endif // STATICALLY_SIZED_ARRAY_FIELDS_SUPPORT

template <> struct TFieldType<AnsiString> { enum { Tid  = int(EFieldType::AnsiString), Size = 0, }; };
template <> struct TFieldType<WideString> { enum { Tid  = int(EFieldType::WideString), Size = 0, }; };



////////////////////////////////////////////////////////////////////////////////
struct FLiteralName
{
	template <uint32 Size>
	explicit FLiteralName(const ANSICHAR (&Name)[Size])
	: Ptr(Name)
	, Length(Size - 1)
	{
		static_assert(Size < 256, "Field name is too large");
	}

	const ANSICHAR* Ptr;
	uint8 Length;
};

////////////////////////////////////////////////////////////////////////////////
struct FFieldDesc
{
	FFieldDesc(const FLiteralName& Name, uint8 Type, uint16 Offset, uint16 Size, uint32 ReferencedUid = 0)
	: Name(Name.Ptr)
	, ValueOffset(Offset)
	, ValueSize(Size)
	, NameSize(Name.Length)
	, TypeInfo(Type)
	, Reference(ReferencedUid)
	{
	}

	const ANSICHAR* Name;
	uint16			ValueOffset;
	uint16			ValueSize;
	uint8			NameSize;
	uint8			TypeInfo;
	uint32			Reference;
};



////////////////////////////////////////////////////////////////////////////////
template <int InIndex, int InOffset, typename Type> struct TField;

enum class EIndexPack
{
	NumFieldsMax	= 1 << FAuxHeader::FieldBits,
	NumFieldsShift	= 8,
	NumFieldsMask	= (1 << NumFieldsShift) - 1,
	AuxFieldCounter	= 1 << NumFieldsShift,
};

////////////////////////////////////////////////////////////////////////////////
#define TRACE_PRIVATE_FIELD(InIndex, InOffset, Type) \
		enum \
		{ \
			Index	= InIndex, \
			Offset	= InOffset, \
			Tid		= TFieldType<Type>::Tid, \
			Size	= TFieldType<Type>::Size, \
		}; \
		static_assert((Index & int(EIndexPack::NumFieldsMask)) < int(EIndexPack::NumFieldsMax), "Trace events may only have up to EIndexPack::NumFieldsMax fields"); \
	private: \
		FFieldDesc FieldDesc; \
	public: \
		TField(const FLiteralName& Name) \
		: FieldDesc(Name, Tid, Offset, Size) \
		{ \
		}

////////////////////////////////////////////////////////////////////////////////
template <int InIndex, int InOffset, typename Type>
struct TField<InIndex, InOffset, Type[]>
{
	TRACE_PRIVATE_FIELD(InIndex + int(EIndexPack::AuxFieldCounter), InOffset, Type[]);
};

#if STATICALLY_SIZED_ARRAY_FIELDS_SUPPORT
////////////////////////////////////////////////////////////////////////////////
template <int InIndex, int InOffset, typename Type, int Count>
struct TField<InIndex, InOffset, Type[Count]>
{
	TRACE_PRIVATE_FIELD(InIndex, InOffset, Type[Count]);
};
#endif // STATICALLY_SIZED_ARRAY_FIELDS_SUPPORT

////////////////////////////////////////////////////////////////////////////////
template <int InIndex, int InOffset>
struct TField<InIndex, InOffset, AnsiString>
{
	TRACE_PRIVATE_FIELD(InIndex + int(EIndexPack::AuxFieldCounter), InOffset, AnsiString);
};

////////////////////////////////////////////////////////////////////////////////
template <int InIndex, int InOffset>
struct TField<InIndex, InOffset, WideString>
{
	TRACE_PRIVATE_FIELD(InIndex + int(EIndexPack::AuxFieldCounter), InOffset, WideString);
};

////////////////////////////////////////////////////////////////////////////////
template <int InIndex, int InOffset, typename DefinitionType>
struct TField<InIndex, InOffset, TEventRef<DefinitionType>>
{
	TRACE_PRIVATE_FIELD(InIndex, InOffset, DefinitionType);
public:
	TField(const FLiteralName& Name, uint32 ReferencedUid)
		: FieldDesc(Name, Tid, Offset, Size, ReferencedUid)
	{
	}
};

////////////////////////////////////////////////////////////////////////////////
template <int InIndex, int InOffset, typename Type>
struct TField
{
	TRACE_PRIVATE_FIELD(InIndex, InOffset, Type);
};

#undef TRACE_PRIVATE_FIELD



////////////////////////////////////////////////////////////////////////////////
// Used to terminate the field list and determine an event's size.
enum EventProps {};
template <int InNumFields, int InSize>
struct TField<InNumFields, InSize, EventProps>
{
	enum : uint16
	{
		NumFields		= InNumFields & int(EIndexPack::NumFieldsMask),
		Size			= InSize,
		NumAuxFields	= (InNumFields >> int(EIndexPack::NumFieldsShift)) & int(EIndexPack::NumFieldsMask),
	};
};

} // namespace Trace
} // namespace UE

#endif // TRACE_PRIVATE_MINIMAL_ENABLED
