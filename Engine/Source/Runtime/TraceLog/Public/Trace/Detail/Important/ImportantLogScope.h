// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Trace/Config.h"

namespace UE {
namespace Trace {
namespace Private {
	
#if TRACE_PRIVATE_MINIMAL_ENABLED && TRACE_PRIVATE_ALLOW_IMPORTANTS

////////////////////////////////////////////////////////////////////////////////
class FImportantLogScope
{
public:
	template <typename EventType>
	static FImportantLogScope	Enter();
	template <typename EventType>
	static FImportantLogScope	Enter(uint32 ArrayDataSize);
	void						operator += (const FImportantLogScope&) const;
	const FImportantLogScope&	operator << (bool) const	{ return *this; }
	constexpr explicit			operator bool () const		{ return true; }

	template <typename FieldMeta, typename Type>
	struct FFieldSet;

private:
	static FImportantLogScope	EnterImpl(uint32 Uid, uint32 Size);
	uint8*						Ptr;
	int32						BufferOffset;
	int32						AuxCursor;
};

#else

class FImportantLogScope
{
public:
	template <typename EventType>
	static FImportantLogScope	Enter() { return FImportantLogScope(); }
	template <typename EventType>
	static FImportantLogScope	Enter(uint32 ArrayDataSize) { return FImportantLogScope(); }
	void						operator += (const FImportantLogScope&) const;
	const FImportantLogScope&	operator << (bool) const	{ return *this; }
	constexpr explicit			operator bool () const		{ return true; }

	template <typename FieldMeta, typename Type>
	struct FFieldSet;
};

#endif // TRACE_PRIVATE_MINIMAL_ENABLED && TRACE_PRIVATE_ALLOW_IMPORTANTS
	
} // namespace Private
} // namespace Trace
} // namespace UE

