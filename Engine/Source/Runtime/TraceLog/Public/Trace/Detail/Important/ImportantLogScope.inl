// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Trace/Config.h"
#include "ImportantLogScope.h"
#include "SharedBuffer.h"
#include "Trace/Detail/Protocol.h"
#include "Trace/Detail/Writer.inl"
#include "Trace/Detail/Field.h"
#include "Trace/Detail/EventNode.h"
#include "AutoRTFM.h"

namespace UE {
namespace Trace {
namespace Private {
	
#if TRACE_PRIVATE_MINIMAL_ENABLED && TRACE_PRIVATE_ALLOW_IMPORTANTS

////////////////////////////////////////////////////////////////////////////////
extern TRACELOG_API FSharedBuffer* volatile GSharedBuffer;
TRACELOG_API FNextSharedBuffer				Writer_NextSharedBuffer(FSharedBuffer*, int32, int32);



////////////////////////////////////////////////////////////////////////////////
template <class T>
UE_AUTORTFM_ALWAYS_OPEN FORCENOINLINE FImportantLogScope FImportantLogScope::Enter(uint32 ArrayDataSize)
{
	static_assert(!!(uint32(T::EventFlags) & uint32(FEventInfo::Flag_MaybeHasAux)), "Only important trace events with array-type fields need a size parameter to UE_TRACE_LOG()");

	ArrayDataSize += sizeof(FAuxHeader) * T::EventProps_Meta::NumAuxFields;
	ArrayDataSize += 1; // for AuxDataTerminal

	uint32 Size = T::GetSize();
	uint32 Uid = T::GetUid() >> EKnownEventUids::_UidShift;
	FImportantLogScope Ret = EnterImpl(Uid, Size + ArrayDataSize);

	Ret.AuxCursor += Size;
	Ret.Ptr[Ret.AuxCursor] = uint8(EKnownEventUids::AuxDataTerminal);
	return Ret;
}

////////////////////////////////////////////////////////////////////////////////
template <class T>
UE_AUTORTFM_ALWAYS_OPEN inline FImportantLogScope FImportantLogScope::Enter()
{
	static_assert(!(uint32(T::EventFlags) & uint32(FEventInfo::Flag_MaybeHasAux)), "Important trace events with array-type fields must be traced with UE_TRACE_LOG(Logger, Event, Channel, ArrayDataSize)");

	uint32 Size = T::GetSize();
	uint32 Uid = T::GetUid() >> EKnownEventUids::_UidShift;
	return EnterImpl(Uid, Size);
}

////////////////////////////////////////////////////////////////////////////////
UE_AUTORTFM_ALWAYS_OPEN inline FImportantLogScope FImportantLogScope::EnterImpl(uint32 Uid, uint32 Size)
{
	FSharedBuffer* Buffer = AtomicLoadAcquire(&GSharedBuffer);

	int32 AllocSize = Size;
	AllocSize += sizeof(FImportantEventHeader);

	// Claim some space in the buffer
	int32 NegSizeAndRef = 0 - ((AllocSize << FSharedBuffer::CursorShift) | FSharedBuffer::RefBit);
	int32 RegionStart = AtomicAddRelaxed(&(Buffer->Cursor), NegSizeAndRef);

	if (UNLIKELY(RegionStart + NegSizeAndRef < 0))
	{
		FNextSharedBuffer Next = Writer_NextSharedBuffer(Buffer, RegionStart, NegSizeAndRef);
		Buffer = Next.Buffer;
		RegionStart = Next.RegionStart;
	}

	int32 Bias = (RegionStart >> FSharedBuffer::CursorShift);
	uint8* Out = (uint8*)Buffer - Bias;

	// Event header
	uint16 Values16[] = { uint16(Uid), uint16(Size) };
	memcpy(Out, Values16, sizeof(Values16)); /* FImportantEventHeader::Uid,Size */

	FImportantLogScope Ret;
	Ret.Ptr = Out + sizeof(FImportantEventHeader);
	Ret.BufferOffset = int32(PTRINT(Buffer) - PTRINT(Ret.Ptr));
	Ret.AuxCursor = 0;
	return Ret;
}

////////////////////////////////////////////////////////////////////////////////
UE_AUTORTFM_ALWAYS_OPEN inline void FImportantLogScope::operator += (const FImportantLogScope& Other) const
{
	auto* Buffer = (FSharedBuffer*)(Ptr + BufferOffset);
	AtomicAddRelease(&(Buffer->Cursor), int32(FSharedBuffer::RefBit));
}

////////////////////////////////////////////////////////////////////////////////
template <typename FieldMeta, typename Type>
struct FImportantLogScope::FFieldSet
{
	UE_AUTORTFM_ALWAYS_OPEN
	static void Impl(FImportantLogScope* Scope, const Type& Value)
	{
		uint8* Dest = (uint8*)(Scope->Ptr) + FieldMeta::Offset;
		::memcpy(Dest, &Value, sizeof(Type));
	}
};

////////////////////////////////////////////////////////////////////////////////
template <typename FieldMeta, typename Type>
struct FImportantLogScope::FFieldSet<FieldMeta, Type[]>
{
	UE_AUTORTFM_ALWAYS_OPEN
	static void Impl(FImportantLogScope* Scope, Type const* Data, int32 Num)
	{
		uint32 Size = Num * sizeof(Type);

		uint32 Pack = Size << FAuxHeader::SizeShift;
		Pack |= (FieldMeta::Index & int32(EIndexPack::NumFieldsMask)) << FAuxHeader::FieldShift;

		uint8* Out = Scope->Ptr + Scope->AuxCursor;
		memcpy(Out, &Pack, sizeof(Pack)); /* FAuxHeader::Pack */
		Out[0] = uint8(EKnownEventUids::AuxData); /* FAuxHeader::Uid */

		memcpy(Out + sizeof(FAuxHeader), Data, Size);

		Scope->AuxCursor += sizeof(FAuxHeader) + Size;
		Scope->Ptr[Scope->AuxCursor] = uint8(EKnownEventUids::AuxDataTerminal);
	}
};

////////////////////////////////////////////////////////////////////////////////
template <typename FieldMeta>
struct FImportantLogScope::FFieldSet<FieldMeta, AnsiString>
{
	UE_AUTORTFM_ALWAYS_OPEN
	static void Impl(FImportantLogScope* Scope, const ANSICHAR* String, int32 Length=-1)
	{
		if (Length < 0)
		{
			Length = int32(strlen(String));
		}

		uint32 Pack = Length << FAuxHeader::SizeShift;
		Pack |= (FieldMeta::Index & int32(EIndexPack::NumFieldsMask)) << FAuxHeader::FieldShift;

		uint8* Out = Scope->Ptr + Scope->AuxCursor;
		memcpy(Out, &Pack, sizeof(Pack)); /* FAuxHeader::FieldIndex_Size */
		Out[0] = uint8(EKnownEventUids::AuxData); /* FAuxHeader::Uid */

		memcpy(Out + sizeof(FAuxHeader), String, Length);

		Scope->AuxCursor += sizeof(FAuxHeader) + Length;
		Scope->Ptr[Scope->AuxCursor] = uint8(EKnownEventUids::AuxDataTerminal);
	}

	UE_AUTORTFM_ALWAYS_OPEN
	static void Impl(FImportantLogScope* Scope, const WIDECHAR* String, int32 Length=-1)
	{
		if (Length < 0)
		{
			Length = 0;
			for (const WIDECHAR* c = String; *c; ++c, ++Length);
		}

		uint32 Pack = Length << FAuxHeader::SizeShift;
		Pack |= (FieldMeta::Index & int32(EIndexPack::NumFieldsMask)) << FAuxHeader::FieldShift;

		uint8* Out = Scope->Ptr + Scope->AuxCursor;
		memcpy(Out, &Pack, sizeof(Pack)); /* FAuxHeader::FieldIndex_Size */
		Out[0] = uint8(EKnownEventUids::AuxData); /* FAuxHeader::Uid */

		Out += sizeof(FAuxHeader);
		for (int32 i = 0; i < Length; ++i)
		{
			*Out = int8(*String);
			++Out;
			++String;
		}

		Scope->AuxCursor += sizeof(FAuxHeader) + Length;
		Scope->Ptr[Scope->AuxCursor] = uint8(EKnownEventUids::AuxDataTerminal);
	}
};

////////////////////////////////////////////////////////////////////////////////
template <typename FieldMeta>
struct FImportantLogScope::FFieldSet<FieldMeta, WideString>
{
	UE_AUTORTFM_ALWAYS_OPEN
	static void Impl(FImportantLogScope* Scope, const WIDECHAR* String, int32 Length=-1)
	{
		if (Length < 0)
		{
			Length = 0;
			for (const WIDECHAR* c = String; *c; ++c, ++Length);
		}

		uint32 Size = Length * sizeof(WIDECHAR);

		uint32 Pack = Size << FAuxHeader::SizeShift;
		Pack |= (FieldMeta::Index & int32(EIndexPack::NumFieldsMask)) << FAuxHeader::FieldShift;

		uint8* Out = Scope->Ptr + Scope->AuxCursor;
		memcpy(Out, &Pack, sizeof(Pack));
		Out[0] = uint8(EKnownEventUids::AuxData);

		memcpy(Out + sizeof(FAuxHeader), String, Size);

		Scope->AuxCursor += sizeof(FAuxHeader) + Size;
		Scope->Ptr[Scope->AuxCursor] = uint8(EKnownEventUids::AuxDataTerminal);
	}
};

////////////////////////////////////////////////////////////////////////////////
template <typename FieldMeta, typename DefinitionType>
struct FImportantLogScope::FFieldSet<FieldMeta, TEventRef<DefinitionType>>
{
	UE_AUTORTFM_ALWAYS_OPEN
	static void Impl(FImportantLogScope* Scope, const TEventRef<DefinitionType>& Reference)
	{
		FFieldSet<FieldMeta, DefinitionType>::Impl(Scope, Reference.Id);
	}
};

#else // TRACE_PRIVATE_MINIMAL_ENABLED && TRACE_PRIVATE_ALLOW_IMPORTANTS
      

template <typename FieldMeta, typename Type>
struct FImportantLogScope::FFieldSet
{
	static void Impl(FImportantLogScope* Scope, const Type& Value)
	{
	}
};
	
template <typename FieldMeta, typename Type>
struct FImportantLogScope::FFieldSet<FieldMeta, Type[]>
{
	static void Impl(FImportantLogScope* Scope, Type const* Data, int32 Num)
	{
	}
};

template <typename FieldMeta>
struct FImportantLogScope::FFieldSet<FieldMeta, AnsiString>
{
	static void Impl(FImportantLogScope* Scope, const ANSICHAR* String, int32 Length=-1)
	{
	}

	static void Impl(FImportantLogScope* Scope, const WIDECHAR* String, int32 Length=-1)
	{
	}
};

template <typename FieldMeta>
struct FImportantLogScope::FFieldSet<FieldMeta, WideString>
{
	static void Impl(FImportantLogScope* Scope, const WIDECHAR* String, int32 Length=-1)
	{
	}
};

template <typename FieldMeta, typename DefinitionType>
struct FImportantLogScope::FFieldSet<FieldMeta, TEventRef<DefinitionType>>
{
	static void Impl(FImportantLogScope* Scope, const TEventRef<DefinitionType>& Reference)
	{
	}
};
	

#endif // TRACE_PRIVATE_MINIMAL_ENABLED && TRACE_PRIVATE_ALLOW_IMPORTANTS

} // namespace Private
} // namespace Trace
} // namespace UE


