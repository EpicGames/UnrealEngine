// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Trace/Config.h"

#if TRACE_PRIVATE_MINIMAL_ENABLED && TRACE_PRIVATE_ALLOW_IMPORTANTS

namespace UE {
namespace Trace {
namespace Private {

////////////////////////////////////////////////////////////////////////////////
struct FSharedBuffer
{
	enum : uint32 {	CursorShift	= 10 };
	enum : uint32 {	RefBit		= 1 << 0 };
	enum : uint32 {	RefInit		= (1 << CursorShift) - 1 };
	enum : uint32 {	MaxSize		= 1 << (32 - CursorShift - 1) };

	int32 volatile	Cursor; // also packs in a ref count.
	uint32			Size;
	uint32			Final;
	uint32			_Unused;
	FSharedBuffer*	Next;
};

////////////////////////////////////////////////////////////////////////////////
struct FNextSharedBuffer
{
	FSharedBuffer*	Buffer;
	int32			RegionStart;
};

} // namespace Private
} // namespace Trace
} // namespace UE

#endif // TRACE_PRIVATE_MINIMAL_ENABLED && TRACE_PRIVATE_ALLOW_IMPORTANTS
