// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Trace/Config.h" // IWYU pragma: export

#if TRACE_PRIVATE_MINIMAL_ENABLED

#include "Writer.inl" // IWYU pragma: export
#include <utility>

namespace UE {
namespace Trace {
namespace Private {

template <bool bMaybeHasAux> class TLogScope;

////////////////////////////////////////////////////////////////////////////////
class FLogScope
{
	friend class FEventNode;

public:
	template <typename EventType>
	static auto				Enter();
	template <typename EventType>
	static auto				ScopedEnter();
	template <typename EventType>
	static auto				ScopedStampedEnter();
	void*					GetPointer() const			{ return Ptr; }
	const FLogScope&		operator << (bool) const	{ return *this; }
	constexpr explicit		operator bool () const		{ return true; }

	template <typename FieldMeta, typename Type>
	struct FFieldSet;

protected:
	void					Commit() const;
	void					Commit(FWriteBuffer* __restrict LatestBuffer) const;

private:
	template <uint32 Flags>
	static auto				EnterImpl(uint32 Uid, uint32 Size);
	template <class T> inline void	EnterPrelude(uint32 Size);
	inline void				Enter(uint32 Uid, uint32 Size, bool bSetSerial = true);
	inline void				EnterNoSync(uint32 Uid, uint32 Size);
	uint8*					Ptr;
	FWriteBuffer*			Buffer;
};

////////////////////////////////////////////////////////////////////////////////
template <bool bMaybeHasAux>
class TLogScope
	: public FLogScope
{
public:
	inline void				operator += (const FLogScope&) const;
};

////////////////////////////////////////////////////////////////////////////////
template <bool bMaybeHasAux>
class TOnConnectLogScope : public TLogScope<bMaybeHasAux>
{
	friend class FLogScope;
public:
	TOnConnectLogScope();
	TOnConnectLogScope(TOnConnectLogScope&& Other)
		: TLogScope<bMaybeHasAux>(std::move(Other))
		, PrevState(Other.PrevState)
		, bActive(Other.bActive)
	{ Other.bActive = false; };
	~TOnConnectLogScope();

private:
	FTlsState PrevState;
	bool bActive;
};


////////////////////////////////////////////////////////////////////////////////
class FScopedLogScope
{
public:
			~FScopedLogScope();
	void	SetActive();
	bool	bActive = false;

private:
	void    Deinit();
};

////////////////////////////////////////////////////////////////////////////////
class FScopedStampedLogScope
{
public:
			~FScopedStampedLogScope();
	void	SetActive();
	bool	bActive = false;

private:
	void    Deinit();
};

} // namespace Private
} // namespace Trace
} // namespace UE

#endif // TRACE_PRIVATE_MINIMAL_ENABLED
