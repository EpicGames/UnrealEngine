// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Trace/Config.h"

#if TRACE_PRIVATE_MINIMAL_ENABLED

#include "Atomic.h"
#include "EventNode.h"
#include "HAL/Platform.h" // for PLATFORM_BREAK
#include "LogScope.h"
#include "Protocol.h"
#include "Writer.inl"
#include "AutoRTFM.h"

namespace UE {
namespace Trace {
namespace Private {

////////////////////////////////////////////////////////////////////////////////
extern TRACELOG_API uint64			GStartCycle;
extern TRACELOG_API uint32 volatile	GLogSerial;

////////////////////////////////////////////////////////////////////////////////
UE_AUTORTFM_ALWAYS_OPEN
inline void FLogScope::Commit() const
{
	AtomicStoreRelease((uint8**) &(Buffer->Committed), Buffer->ReadCursor());
}

////////////////////////////////////////////////////////////////////////////////
UE_AUTORTFM_ALWAYS_OPEN
inline void FLogScope::Commit(FWriteBuffer* __restrict LatestBuffer) const
{
	if (LatestBuffer != Buffer)
	{
		AtomicStoreRelease((uint8**) &(LatestBuffer->Committed), LatestBuffer->ReadCursor());
	}

	Commit();
}

////////////////////////////////////////////////////////////////////////////////
template <uint32 Flags>
UE_AUTORTFM_ALWAYS_OPEN
inline auto FLogScope::EnterImpl(uint32 Uid, uint32 Size)
{
	constexpr bool bMaybeHasAux = (Flags & FEventInfo::Flag_MaybeHasAux) != 0;

	if constexpr ((Flags & FEventInfo::Flag_MaybeImportant) != 0)
	{
		// If a on connect buffer queue is active this scope
		// will redirect the event to this separate queue. If not
		// it will act exactly like TLogScope.
		TOnConnectLogScope<bMaybeHasAux> Ret;
		if ((Flags & FEventInfo::Flag_NoSync) != 0)
		{
			Ret.EnterNoSync(Uid, Size);
		}
		else
		{
			// When the TOnConnectLogScope is active we're in an OnConnect
			// callback and this event moved ahead of the tail which means 
			// we don't set the serial number for synchronized events. 
			const bool bSetSerial = Ret.bActive ? false : true;
			Ret.Enter(Uid, Size, bSetSerial);
		}
		return Ret;
	}
	else
	{
		TLogScope<bMaybeHasAux> Ret;
		if ((Flags & FEventInfo::Flag_NoSync) != 0)
		{
			Ret.EnterNoSync(Uid, Size);
		}
		else
		{
			Ret.Enter(Uid, Size);
		}
		return Ret;
	}
}

////////////////////////////////////////////////////////////////////////////////
template <class HeaderType>
UE_AUTORTFM_ALWAYS_OPEN
inline void FLogScope::EnterPrelude(uint32 Size)
{
	uint32 AllocSize = sizeof(HeaderType) + Size;

	Buffer = Writer_GetBuffer();
	if (UNLIKELY(Buffer->ReadCursor() + AllocSize > (uint8*)Buffer))
	{
		Buffer = Writer_NextBuffer(Buffer);
	}

#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST
	if (AllocSize >= Buffer->Size)
	{
		// This situation is terminal. Someone's trying to trace an event that
		// is far too large. This should never happen as 'maximum_field_count *
		// largest_field_size' won't exceed Buffer.Size.
		PLATFORM_BREAK();
	}
#endif

	Ptr = Buffer->ReadCursor() + sizeof(HeaderType);
	Buffer->MoveCursor((uint16)AllocSize);
}

////////////////////////////////////////////////////////////////////////////////
UE_AUTORTFM_ALWAYS_OPEN
inline void FLogScope::Enter(uint32 Uid, uint32 Size, bool bSetSerial /* = true */)
{
	EnterPrelude<FEventHeaderSync>(Size);

	uint16 Uid16 = uint16(Uid) | int32(EKnownEventUids::Flag_TwoByteUid);
	uint32 Serial = bSetSerial ? uint32(AtomicAddRelaxed(&GLogSerial, 1u)) : 0u;

	// Event header FEventHeaderSync
	memcpy(Ptr - 3, &Serial, sizeof(Serial)); /* FEventHeaderSync::SerialHigh,SerialLow */
	memcpy(Ptr - 5, &Uid16,  sizeof(Uid16));  /* FEventHeaderSync::Uid */
}

////////////////////////////////////////////////////////////////////////////////
UE_AUTORTFM_ALWAYS_OPEN
inline void FLogScope::EnterNoSync(uint32 Uid, uint32 Size)
{
	EnterPrelude<FEventHeader>(Size);

	uint16 Uid16 = uint16(Uid) | int32(EKnownEventUids::Flag_TwoByteUid);

	// Event header FEventHeader
	auto* Header = (uint16*)(Ptr);
	memcpy(Header - 1, &Uid16, sizeof(Uid16)); /* FEventHeader::Uid */
}

////////////////////////////////////////////////////////////////////////////////
template <bool bMaybeHasAux>
UE_AUTORTFM_ALWAYS_OPEN
inline void TLogScope<bMaybeHasAux>::operator += (const FLogScope&) const
{
	if constexpr (bMaybeHasAux)
	{
		FWriteBuffer* LatestBuffer = Writer_GetBuffer();
		*LatestBuffer->ReadCursor() = uint8(EKnownEventUids::AuxDataTerminal << EKnownEventUids::_UidShift);
		LatestBuffer->MoveCursor(1u);

		Commit(LatestBuffer);
	}
	else
	{
		Commit();
	}
}

////////////////////////////////////////////////////////////////////////////////
inline void FScopedLogScope::Deinit()
{
	if (!bActive)
	{
		return;
	}

	if (AutoRTFM::IsClosed())
	{
		// If we are operating closed (in a transaction) we have to forget about
		// our previously registered on-abort handler, because we're currently
		// about to close out the scope within the transaction.
		AutoRTFM::PopOnAbortHandler(this);
		AutoRTFM::Open([this] { this->Deinit(); });
		return;
	}

	uint8 LeaveUid = uint8(EKnownEventUids::LeaveScope << EKnownEventUids::_UidShift);

	FWriteBuffer* Buffer = Writer_GetBuffer();
	if (UNLIKELY(int32((uint8*)Buffer - Buffer->ReadCursor()) < int32(sizeof(LeaveUid))))
	{
		Buffer = Writer_NextBuffer(Buffer);
	}

	*Buffer->ReadCursor() = LeaveUid;
	Buffer->MoveCursor(sizeof(LeaveUid));

	AtomicStoreRelease((uint8**) &(Buffer->Committed), Buffer->ReadCursor());
}

////////////////////////////////////////////////////////////////////////////////
inline FScopedLogScope::~FScopedLogScope()
{
	Deinit();
}

////////////////////////////////////////////////////////////////////////////////
inline void FScopedLogScope::SetActive()
{
	if (bActive)
	{
		// We don't want to re-activate an already active scope!
		return;
	}

	if (AutoRTFM::IsClosed())
	{
		// If we are operating closed (in a transaction) we could abort before
		// `this` hits its destructor. In that case we need to remember when
		// we abort to close out the scoped trace event.
		AutoRTFM::PushOnAbortHandler(this, [this] { this->Deinit(); });
		AutoRTFM::Open([this] { this->SetActive(); });
		return;
	}

	bActive = true;
}

////////////////////////////////////////////////////////////////////////////////
inline void FScopedStampedLogScope::Deinit()
{
	if (!bActive)
	{
		return;
	}

	if (AutoRTFM::IsClosed())
	{
		// If we are operating closed (in a transaction) we have to forget about
		// our previously registered on-abort handler, because we're currently
		// about to close out the scope within the transaction.
		AutoRTFM::PopOnAbortHandler(this);
		AutoRTFM::Open([this] { this->Deinit(); });
		return;
	}

	uint64 Stamp = TimeGetTimestamp() - GStartCycle;

	FWriteBuffer* Buffer = Writer_GetBuffer();
	if (UNLIKELY(int32((uint8*)Buffer - Buffer->ReadCursor()) < int32(sizeof(Stamp))))
	{
		Buffer = Writer_NextBuffer(Buffer);
	}

	Stamp <<= 8;
	Stamp += uint8(EKnownEventUids::LeaveScope_TB) << EKnownEventUids::_UidShift;
	memcpy(Buffer->ReadCursor(), &Stamp, sizeof(Stamp));
	Buffer->MoveCursor(sizeof(Stamp));

	AtomicStoreRelease((uint8**) &(Buffer->Committed), Buffer->ReadCursor());
}

////////////////////////////////////////////////////////////////////////////////
inline FScopedStampedLogScope::~FScopedStampedLogScope()
{
	Deinit();
}

////////////////////////////////////////////////////////////////////////////////
inline void FScopedStampedLogScope::SetActive()
{
	if (bActive)
	{
		// We don't want to re-activate an already active scope!
		return;
	}

	if (AutoRTFM::IsClosed())
	{
		// If we are operating closed (in a transaction) we could abort before
		// `this` hits its destructor. In that case we need to remember when
		// we abort to close out the scoped trace event.
		AutoRTFM::PushOnAbortHandler(this, [this] { this->Deinit(); });
		AutoRTFM::Open([this] { this->SetActive(); });
		return;
	}

	bActive = true;
}

////////////////////////////////////////////////////////////////////////////////
template <class EventType>
UE_AUTORTFM_ALWAYS_OPEN
FORCENOINLINE auto FLogScope::Enter()
{
	uint32 Size = EventType::GetSize();
	uint32 Uid = EventType::GetUid();
	return EnterImpl<EventType::EventFlags>(Uid, Size);
}

////////////////////////////////////////////////////////////////////////////////
template <class EventType>
UE_AUTORTFM_ALWAYS_OPEN
FORCENOINLINE auto FLogScope::ScopedEnter()
{
	uint8 EnterUid = uint8(EKnownEventUids::EnterScope << EKnownEventUids::_UidShift);

	FWriteBuffer* Buffer = Writer_GetBuffer();
	constexpr int32 EventHeaderSize = (EventType::EventFlags & FEventInfo::Flag_NoSync) != 0 ? sizeof(FEventHeader) : sizeof(FEventHeaderSync);
	constexpr int32 RequiredSize = sizeof(EnterUid) + EventHeaderSize + EventType::GetSize();
	if (UNLIKELY(int32((uint8*)Buffer - Buffer->ReadCursor()) < RequiredSize))
	{
		Buffer = Writer_NextBuffer(Buffer);
	}

	*Buffer->ReadCursor() = EnterUid;
	Buffer->MoveCursor(sizeof(EnterUid));

	AtomicStoreRelease((uint8**) &(Buffer->Committed), Buffer->ReadCursor());

	return Enter<EventType>();
}

////////////////////////////////////////////////////////////////////////////////
template <class EventType>
UE_AUTORTFM_ALWAYS_OPEN
FORCENOINLINE auto FLogScope::ScopedStampedEnter()
{
	uint64 Stamp;

	FWriteBuffer* Buffer = Writer_GetBuffer();
	constexpr int32 EventHeaderSize = (EventType::EventFlags & FEventInfo::Flag_NoSync) != 0 ? sizeof(FEventHeader) : sizeof(FEventHeaderSync);
	constexpr int32 RequiredSize = sizeof(Stamp) + EventHeaderSize + EventType::GetSize();
	if (UNLIKELY(int32((uint8*)Buffer - Buffer->ReadCursor()) < RequiredSize))
	{
		Buffer = Writer_NextBuffer(Buffer);
	}

	Stamp = TimeGetTimestamp() - GStartCycle;
	Stamp <<= 8;
	Stamp += uint8(EKnownEventUids::EnterScope_TB) << EKnownEventUids::_UidShift;
	memcpy(Buffer->ReadCursor(), &Stamp, sizeof(Stamp));
	Buffer->MoveCursor(sizeof(Stamp));

	AtomicStoreRelease((uint8**) &(Buffer->Committed), Buffer->ReadCursor());

	return Enter<EventType>();
}



////////////////////////////////////////////////////////////////////////////////
template <typename FieldMeta, typename Type>
struct FLogScope::FFieldSet
{
	UE_AUTORTFM_ALWAYS_OPEN
	static void Impl(FLogScope* Scope, const Type& Value)
	{
		uint8* Dest = (uint8*)(Scope->Ptr) + FieldMeta::Offset;
		::memcpy(Dest, &Value, sizeof(Type));
	}
};

////////////////////////////////////////////////////////////////////////////////
template <typename FieldMeta, typename Type>
struct FLogScope::FFieldSet<FieldMeta, Type[]>
{
	UE_AUTORTFM_ALWAYS_OPEN
	static void Impl(FLogScope*, Type const* Data, int32 Num)
	{
		static const uint32 Index = FieldMeta::Index & int32(EIndexPack::NumFieldsMask);
		int32 Size = (Num * sizeof(Type)) & (FAuxHeader::SizeLimit - 1) & ~(sizeof(Type) - 1);
		Field_WriteAuxData(Index, (const uint8*)Data, Size);
	}
};

#if STATICALLY_SIZED_ARRAY_FIELDS_SUPPORT
////////////////////////////////////////////////////////////////////////////////
template <typename FieldMeta, typename Type, int32 Count>
struct FLogScope::FFieldSet<FieldMeta, Type[Count]>
{
	static void Impl(FLogScope*, Type const* Data, int32 Num=-1) = delete;
};
#endif // STATICALLY_SIZED_ARRAY_FIELDS_SUPPORT

////////////////////////////////////////////////////////////////////////////////
template <typename FieldMeta>
struct FLogScope::FFieldSet<FieldMeta, AnsiString>
{
	UE_AUTORTFM_ALWAYS_OPEN
	static void Impl(FLogScope*, const ANSICHAR* String, int32 Length=-1)
	{
		if (Length < 0)
		{
			Length = int32(strlen(String));
		}

		static const uint32 Index = FieldMeta::Index & int32(EIndexPack::NumFieldsMask);
		Field_WriteStringAnsi(Index, String, Length);
	}
	
	UE_AUTORTFM_ALWAYS_OPEN
	static void Impl(FLogScope*, const WIDECHAR* String, int32 Length=-1)
	{
		if (Length < 0)
		{
			Length = 0;
			for (const WIDECHAR* c = String; *c; ++c, ++Length);
		}

		static const uint32 Index = FieldMeta::Index & int32(EIndexPack::NumFieldsMask);
		Field_WriteStringAnsi(Index, String, Length);
	}
};

////////////////////////////////////////////////////////////////////////////////
template <typename FieldMeta>
struct FLogScope::FFieldSet<FieldMeta, WideString>
{
	UE_AUTORTFM_ALWAYS_OPEN
	static void Impl(FLogScope*, const WIDECHAR* String, int32 Length=-1)
	{
		if (Length < 0)
		{
			Length = 0;
			for (const WIDECHAR* c = String; *c; ++c, ++Length);
		}

		static const uint32 Index = FieldMeta::Index & int32(EIndexPack::NumFieldsMask);
		Field_WriteStringWide(Index, String, Length);
	}
};

////////////////////////////////////////////////////////////////////////////////
template <typename FieldMeta, typename DefinitionType>
struct FLogScope::FFieldSet<FieldMeta, TEventRef<DefinitionType>>
{
	UE_AUTORTFM_ALWAYS_OPEN
	static void Impl(FLogScope* Scope, const TEventRef<DefinitionType>& Reference)
	{
		FFieldSet<FieldMeta, DefinitionType>::Impl(Scope, Reference.Id);
	}
};

////////////////////////////////////////////////////////////////////////////////
template <bool bMaybeHasAux>
TOnConnectLogScope<bMaybeHasAux>::TOnConnectLogScope()
{
	bActive = Writer_OnConnectActivate(PrevState);
}

////////////////////////////////////////////////////////////////////////////////
template <bool bMaybeHasAux>
TOnConnectLogScope<bMaybeHasAux>::~TOnConnectLogScope()
{
	if (bActive)
	{
		Writer_OnConnectDeactivate(PrevState);
	}
}




} // namespace Private
} // namespace Trace
} // namespace UE

#endif // TRACE_PRIVATE_MINIMAL_ENABLED
