// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Trace/Config.h"

////////////////////////////////////////////////////////////////////////////////
#if defined(_MSC_VER)
	#pragma warning(push)
	#pragma warning(disable : 4200) // non-standard zero-sized array
#endif

// IWYU pragma: begin_exports
#include "Protocols/Protocol0.h"
#include "Protocols/Protocol1.h"
#include "Protocols/Protocol2.h"
#include "Protocols/Protocol3.h"
#include "Protocols/Protocol4.h"
#include "Protocols/Protocol5.h"
#include "Protocols/Protocol6.h"
#include "Protocols/Protocol7.h"
// IWYU pragma: end_exports

#if defined(_MSC_VER)
	#pragma warning(pop)
#endif
