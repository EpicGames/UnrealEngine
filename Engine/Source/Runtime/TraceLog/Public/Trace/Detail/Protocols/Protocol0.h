// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Not included directly

namespace UE {
namespace Trace {

#if defined(TRACE_PRIVATE_PROTOCOL_0)
inline
#endif
namespace Protocol0
{

////////////////////////////////////////////////////////////////////////////////
enum EProtocol : uint8 { Id = 0 };

////////////////////////////////////////////////////////////////////////////////
enum : uint8
{
	/* Category */
	Field_CategoryMask	= 0300,
	Field_Integer		= 0000,
	Field_Float			= 0100,
	Field_Array			= 0200,

	/* Size */
	Field_Pow2SizeMask	= 0003,
	Field_8				= 0000,
	Field_16			= 0001,
	Field_32			= 0002,
	Field_64			= 0003,
	Field_Ptr			= Field_64,

	/* Specials */
	Field_SpecialMask	= 0030,
	Field_Pod			= 0000,
	Field_String		= 0010,
	Field_Signed		= 0020,
	/*Field_Unused_3	= 0030,
	  ...
	  Field_Unused_7	= 0070,*/
};

////////////////////////////////////////////////////////////////////////////////
enum class EFieldType : uint8
{
	Bool		= Field_Pod    | Field_Integer                 | Field_8,
	Int8		= Field_Pod    | Field_Integer|Field_Signed    | Field_8,
	Int16		= Field_Pod    | Field_Integer|Field_Signed    | Field_16,
	Int32		= Field_Pod    | Field_Integer|Field_Signed    | Field_32,
	Int64		= Field_Pod    | Field_Integer|Field_Signed    | Field_64,
	Uint8		= Field_Pod    | Field_Integer                 | Field_8,
	Uint16		= Field_Pod    | Field_Integer                 | Field_16,
	Uint32		= Field_Pod    | Field_Integer                 | Field_32,
	Uint64		= Field_Pod    | Field_Integer                 | Field_64,
	Pointer		= Field_Pod    | Field_Integer                 | Field_Ptr,
	Float32		= Field_Pod    | Field_Float                   | Field_32,
	Float64		= Field_Pod    | Field_Float                   | Field_64,
	AnsiString	= Field_String | Field_Integer|Field_Array     | Field_8,
	WideString	= Field_String | Field_Integer|Field_Array     | Field_16,
	Array		= Field_Array,
};

////////////////////////////////////////////////////////////////////////////////
struct FNewEventEvent
{
	uint16		EventUid;
	uint8		FieldCount;
	uint8		Flags;
	uint8		LoggerNameSize;
	uint8		EventNameSize;
	struct
	{
		uint16	Offset;
		uint16	Size;
		uint8	TypeInfo;
		uint8	NameSize;
	}			Fields[];
	/*uint8		NameData[]*/
};

////////////////////////////////////////////////////////////////////////////////
enum class EKnownEventUids : uint16
{
	NewEvent,
	User,
	Max				= (1 << 14) - 1, // ...leaves two MSB bits for other uses.
	UidMask			= Max,
	Invalid			= Max,
	Flag_Important	= 1 << 14,
	Flag_Unused		= 1 << 15,
};

////////////////////////////////////////////////////////////////////////////////
struct FEventHeader
{
	uint16		Uid;
	uint16		Size;
	uint8		EventData[];
};

} // namespace Protocol0
} // namespace Trace
} // namespace UE
