// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Not included directly

namespace UE {
namespace Trace {

#if defined(TRACE_PRIVATE_PROTOCOL_1)
inline
#endif
namespace Protocol1
{

////////////////////////////////////////////////////////////////////////////////
enum EProtocol : uint8 { Id = 1 };

////////////////////////////////////////////////////////////////////////////////
using Protocol0::EFieldType;
using Protocol0::FNewEventEvent;

////////////////////////////////////////////////////////////////////////////////
enum class EEventFlags : uint8
{
	Important		= 1 << 0,
	MaybeHasAux		= 1 << 1,
	NoSync			= 1 << 2,
};

////////////////////////////////////////////////////////////////////////////////
enum class EKnownEventUids : uint16
{
	NewEvent,
	User,
	Max			= (1 << 15) - 1,
	UidMask		= Max,
	Invalid		= Max,
};

////////////////////////////////////////////////////////////////////////////////
struct FEventHeader
{
	uint16		Uid;
	uint16		Size;
	uint16		Serial;
	uint8		EventData[];
};

////////////////////////////////////////////////////////////////////////////////
struct FAuxHeader
{
	enum : uint32
	{
		AuxDataBit	= 0x80,
		FieldMask	= 0x7f,
		SizeLimit	= 1 << 24,
	};

	union
	{
		uint8	FieldIndex;	// 7 bits max (MSB is used to indicate aux data)
		uint32	Size;		// encoded as (Size & 0x00ffffff) << 8
	};
	uint8		Data[];
};

} // namespace Protocol1
} // namespace Trace
} // namespace UE
