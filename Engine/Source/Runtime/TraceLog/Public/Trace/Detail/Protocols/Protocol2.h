// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Not included directly

namespace UE {
namespace Trace {

#if defined(TRACE_PRIVATE_PROTOCOL_2)
inline
#endif
namespace Protocol2
{

////////////////////////////////////////////////////////////////////////////////
enum EProtocol : uint8 { Id = 2 };

////////////////////////////////////////////////////////////////////////////////
using Protocol1::EFieldType;
using Protocol1::FNewEventEvent;
using Protocol1::EEventFlags;
using Protocol1::EKnownEventUids;
using Protocol1::FAuxHeader;

////////////////////////////////////////////////////////////////////////////////
struct FEventHeader
{
	uint16		Uid;
	uint16		Size;
};

////////////////////////////////////////////////////////////////////////////////
#pragma pack(push, 1)
struct FEventHeaderSync
	: public FEventHeader
{
	uint16		SerialLow;		// 24-bit...
	uint8		SerialHigh;		// ...serial no.
	uint8		EventData[];
};
#pragma pack(pop)
static_assert(sizeof(FEventHeaderSync) == 7, "Packing assumption doesn't hold");

} // namespace Protocol2
} // namespace Trace
} // namespace UE
