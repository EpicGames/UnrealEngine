// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Not included directly

namespace UE {
namespace Trace {

#if defined(TRACE_PRIVATE_PROTOCOL_3)
inline
#endif
namespace Protocol3
{

////////////////////////////////////////////////////////////////////////////////
enum EProtocol : uint8 { Id = 3 };

////////////////////////////////////////////////////////////////////////////////
using Protocol2::EFieldType;
using Protocol2::FNewEventEvent;
using Protocol2::EEventFlags;
using Protocol2::EKnownEventUids;
using Protocol2::FAuxHeader;
using Protocol2::FEventHeader;
using Protocol2::FEventHeaderSync;

} // namespace Protocol3
} // namespace Trace
} // namespace UE
