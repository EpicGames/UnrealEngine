// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Not included directly

namespace UE {
namespace Trace {

#if defined(TRACE_PRIVATE_PROTOCOL_4)
inline
#endif
namespace Protocol4
{

////////////////////////////////////////////////////////////////////////////////
enum EProtocol : uint8 { Id = 4 };

////////////////////////////////////////////////////////////////////////////////
using Protocol3::EFieldType;
using Protocol3::FNewEventEvent;
using Protocol3::EEventFlags;
using Protocol3::FAuxHeader;
using Protocol3::FEventHeader;
using Protocol3::FEventHeaderSync;

////////////////////////////////////////////////////////////////////////////////
struct EKnownEventUids
{
	static const uint16 Flag_TwoByteUid	= 1 << 0;
	static const uint16 _UidShift		= 1;
	enum : uint16
	{
		NewEvent						= 0,
		EnterScope,
		EnterScope_T,
		LeaveScope,
		LeaveScope_T,
		_WellKnownNum,
	};
	static const uint16 User			= _WellKnownNum;
	static const uint16 Max				= (1 << (16 - _UidShift)) - 1;
	static const uint16 Invalid			= Max;
};

} // namespace Protocol4
} // namespace Trace
} // namespace UE
