// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Not included directly

namespace UE {
namespace Trace {

#if defined(TRACE_PRIVATE_PROTOCOL_5)
inline
#endif
namespace Protocol5
{

////////////////////////////////////////////////////////////////////////////////
enum EProtocol : uint8 { Id = 5 };

////////////////////////////////////////////////////////////////////////////////
using Protocol4::EFieldType;
using Protocol4::FNewEventEvent;
using Protocol4::EEventFlags;

////////////////////////////////////////////////////////////////////////////////
struct EKnownEventUids
{
	static const uint16 Flag_TwoByteUid	= 1 << 0;
	static const uint16 _UidShift		= 1;
	enum : uint16
	{
		NewEvent						= 0,
		AuxData,
		_AuxData_Unused,
		AuxDataTerminal,
		EnterScope,
		LeaveScope,
		_Unused6,
		_Unused7,
		EnterScope_T,
		_EnterScope_T_Unused0,	// reserved for variable
		_EnterScope_T_Unused1,	// length timestamps
		_EnterScope_T_Unused2,
		LeaveScope_T,
		_LeaveScope_T_Unused0,
		_LeaveScope_T_Unused1,
		_LeaveScope_T_Unused2,
		_WellKnownNum,
	};
	static const uint16 User			= _WellKnownNum;
	static const uint16 Max				= (1 << (16 - _UidShift)) - 1;
	static const uint16 Invalid			= Max;
};

////////////////////////////////////////////////////////////////////////////////
struct FEventHeader
{
	uint16		Uid;
	uint8		Data[];
};
static_assert(sizeof(FEventHeader) == 2, "Struct layout assumption doesn't match expectation");

////////////////////////////////////////////////////////////////////////////////
struct FImportantEventHeader
{
	uint16		Uid;
	uint16		Size;
	uint8		Data[];
};
static_assert(sizeof(FImportantEventHeader) == 4, "Struct layout assumption doesn't match expectation");

////////////////////////////////////////////////////////////////////////////////
#pragma pack(push, 1)
struct FEventHeaderSync
{
	uint16		Uid;
	uint16		SerialLow;		// 24-bit
	uint8		SerialHigh;		// serial no.
	uint8		Data[];
};
#pragma pack(pop)
static_assert(sizeof(FEventHeaderSync) == 5, "Packing assumption doesn't hold");

////////////////////////////////////////////////////////////////////////////////
struct FAuxHeader
{
	enum : uint32
	{
		FieldShift	= 8,
		FieldBits	= 5,
		FieldMask	= (1 << FieldBits) - 1,
		SizeShift	= FieldShift + FieldBits,
		SizeLimit	= 1 << (32 - SizeShift),
	};

	union
	{
		struct
		{
			uint8	Uid;
			uint8	FieldIndex_Size;
			uint16	Size;
		};
		uint32		Pack;
	};
	uint8		Data[];
};
static_assert(sizeof(FAuxHeader) == 4, "Struct layout assumption doesn't match expectation");

} // namespace Protocol5
} // namespace Trace
} // namespace UE
