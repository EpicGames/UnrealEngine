// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Not included directly

namespace UE {
namespace Trace {

#if defined(TRACE_PRIVATE_PROTOCOL_6)
inline
#endif
namespace Protocol6
{

////////////////////////////////////////////////////////////////////////////////
enum EProtocol : uint8 { Id = 6 };

////////////////////////////////////////////////////////////////////////////////
using Protocol5::EFieldType;
using Protocol5::FEventHeader;
using Protocol5::FImportantEventHeader;
using Protocol5::FEventHeaderSync;
using Protocol5::FAuxHeader;
using Protocol5::EKnownEventUids;

////////////////////////////////////////////////////////////////////////////////
enum class EEventFlags : uint8
{
	None			= 0,
	Important		= 1 << 0,
	MaybeHasAux		= 1 << 1,
	NoSync			= 1 << 2,
	Definition		= 1 << 3,
};

////////////////////////////////////////////////////////////////////////////////
enum EFieldFamily : uint8
{
	Regular,
	Reference,
	DefinitionId,
};
	
////////////////////////////////////////////////////////////////////////////////
struct FNewEventField
{
	uint8	FieldType; // EFieldFamily
	uint8	Unused;
	union
	{
		struct
		{
			uint16	Offset;
			uint16	Size;
			uint8	TypeInfo; // EFieldType
			uint8	NameSize;
		} Regular;
		struct
		{
			uint16	Offset;
			uint16	RefUid;
			uint8	TypeInfo; // EFieldType
			uint8	NameSize;
		} Reference;
		struct
		{
			uint16	Offset;
			uint16	Unused1;
			uint8	Unused2;
			uint8	TypeInfo; // EFieldType
		} DefinitionId;
	};
};
static_assert(sizeof(FNewEventField) == 8, "Struct layout assumption doesn't match expectation");

struct FNewEventEvent
{
	uint16			EventUid;
	uint8			FieldCount;
	uint8			Flags; // EEventFlags
	uint8			LoggerNameSize;
	uint8			EventNameSize;
	FNewEventField	Fields[];
};
static_assert(sizeof(FNewEventEvent) == 6, "Struct layout assumption doesn't match expectation");
	
////////////////////////////////////////////////////////////////////////////////

} // namespace Protocol6
} // namespace Trace
} // namespace UE
