// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Not included directly

namespace UE {
namespace Trace {

#if defined(TRACE_PRIVATE_PROTOCOL_7)
inline
#endif
namespace Protocol7
{

////////////////////////////////////////////////////////////////////////////////
enum EProtocol : uint8 { Id = 7 };

////////////////////////////////////////////////////////////////////////////////
using Protocol6::EFieldType;
using Protocol6::FEventHeader;
using Protocol6::FImportantEventHeader;
using Protocol6::FEventHeaderSync;
using Protocol6::FAuxHeader;
using Protocol6::EEventFlags;
using Protocol6::EFieldFamily;
using Protocol6::FNewEventField;
using Protocol6::FNewEventEvent;

////////////////////////////////////////////////////////////////////////////////
struct EKnownEventUids
{
	static const uint16 Flag_TwoByteUid = 1 << 0;
	static const uint16 _UidShift = 1;
	enum : uint16
	{
		NewEvent = 0,			// same as Protocol5
		AuxData,				// same as Protocol5
		_Unused0,
		AuxDataTerminal,		// same as Protocol5
		EnterScope,				// same as Protocol5
		LeaveScope,				// same as Protocol5
		EnterScope_TA,			// new in Protocol7, absolute timestamps
		LeaveScope_TA,			// new in Protocol7, absolute timestamps
		EnterScope_TB,			// new in Protocol7, timestamps relative to BaseTimestamp
		LeaveScope_TB,			// new in Protocol7, timestamps relative to BaseTimestamp
		_Unused1,
		_Unused2,
		_Unused3,
		_Unused4,
		_Unused5,
		_Unused6,
		_WellKnownNum,			// same as Protocol5
	};
	static const uint16 User = _WellKnownNum;
	static const uint16 Max = (1 << (16 - _UidShift)) - 1;
	static const uint16 Invalid = Max;
	static_assert(User == Protocol5::EKnownEventUids::User, "Protocol7::EKnownEventUids should extend Protocol5");
};

////////////////////////////////////////////////////////////////////////////////

} // namespace Protocol7
} // namespace Trace
} // namespace UE
