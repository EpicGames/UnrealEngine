// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// IWYU pragma: begin_exports

#include "Trace/Config.h"


#include "Misc/Launder.h"
#include "Misc/CoreMiscDefines.h"

#include <type_traits>

// IWYU pragma: end_exports

namespace UE {
namespace Trace {

class FChannel;

} // namespace Trace
} // namespace UE

#define TRACE_PRIVATE_EXPAND(x) x

#if TRACE_PRIVATE_MINIMAL_ENABLED

//------------------------------------------------------------------------------------------------
//-- Private functional variants
//------------------------------------------------------------------------------------------------

#define TRACE_PRIVATE_STATISTICS UE_TRACE_STATISTICS_ENABLED

#define TRACE_PRIVATE_CHANNEL_DEFAULT_ARGS false, "None"

#define TRACE_PRIVATE_CHANNEL_DECLARE(LinkageType, ChannelName, ChannelClass) \
	static_assert(std::is_base_of_v<UE::Trace::FChannel, ChannelClass>);\
	static ChannelClass ChannelName##Object; \
	LinkageType ChannelClass& ChannelName = ChannelName##Object;

#define TRACE_PRIVATE_CHANNEL_IMPL(ChannelName, ...) \
	struct F##ChannelName##Registrator \
	{ \
		F##ChannelName##Registrator() \
		{ \
			ChannelName##Object.Setup(#ChannelName, { __VA_ARGS__ } ); \
		} \
	}; \
	static F##ChannelName##Registrator ChannelName##Reg = F##ChannelName##Registrator();

#define TRACE_PRIVATE_CHANNEL(ChannelName, ...) \
	TRACE_PRIVATE_CHANNEL_DECLARE(static, ChannelName, UE::Trace::FChannel) \
	TRACE_PRIVATE_CHANNEL_IMPL(ChannelName, ##__VA_ARGS__)

#define TRACE_PRIVATE_CHANNEL_CUSTOM(ChannelName, ChannelClass, ...) \
	TRACE_PRIVATE_CHANNEL_DECLARE(static, ChannelName, ChannelClass) \
	TRACE_PRIVATE_CHANNEL_IMPL(ChannelName, ##__VA_ARGS__)

#define TRACE_PRIVATE_CHANNEL_DEFINE(ChannelName, ...) \
	TRACE_PRIVATE_CHANNEL_DECLARE(, ChannelName, UE::Trace::FChannel) \
	TRACE_PRIVATE_CHANNEL_IMPL(ChannelName, ##__VA_ARGS__)

#define TRACE_PRIVATE_CHANNEL_CUSTOM_DEFINE(ChannelName, ChannelClass, ...) \
	TRACE_PRIVATE_CHANNEL_DECLARE(, ChannelName, ChannelClass) \
	TRACE_PRIVATE_CHANNEL_IMPL(ChannelName, ##__VA_ARGS__)

#define TRACE_PRIVATE_CHANNEL_EXTERN(ChannelName, ...) \
	__VA_ARGS__ extern UE::Trace::FChannel& ChannelName;

#define TRACE_PRIVATE_CHANNEL_CUSTOM_EXTERN(ChannelName, ChannelClass, ...) \
	__VA_ARGS__ extern class ChannelClass& ChannelName;

#define TRACE_PRIVATE_CHANNELEXPR_IS_ENABLED(ChannelsExpr) \
	bool(ChannelsExpr)

#define TRACE_PRIVATE_EVENT_DEFINE(LoggerName, EventName) \
	static UE::Trace::Private::FEventNode LoggerName##EventName##Event##Impl;\
	UE::Trace::Private::FEventNode& LoggerName##EventName##Event = LoggerName##EventName##Event##Impl;

#define TRACE_PRIVATE_EVENT_BEGIN(LoggerName, EventName, ...) \
	TRACE_PRIVATE_EVENT_DEFINE(LoggerName, EventName); \
	TRACE_PRIVATE_EVENT_BEGIN_IMPL(LoggerName, EventName, ##__VA_ARGS__)

#define TRACE_PRIVATE_EVENT_BEGIN_EXTERN(LoggerName, EventName, ...) \
	extern UE::Trace::Private::FEventNode& LoggerName##EventName##Event; \
	TRACE_PRIVATE_EVENT_BEGIN_IMPL(LoggerName, EventName, ##__VA_ARGS__)

#define TRACE_PRIVATE_EVENT_BEGIN_IMPL(LoggerName, EventName, ...) \
	struct F##LoggerName##EventName##Fields \
	{ \
		enum \
		{ \
			Important			= UE::Trace::Private::FEventInfo::Flag_Important, \
			NoSync				= UE::Trace::Private::FEventInfo::Flag_NoSync, \
			MaybeImportant		= UE::Trace::Private::FEventInfo::Flag_MaybeImportant, \
			Definition8bit		= UE::Trace::Private::FEventInfo::Flag_Definition8, \
			Definition16bit		= UE::Trace::Private::FEventInfo::Flag_Definition16, \
			Definition32bit		= UE::Trace::Private::FEventInfo::Flag_Definition32, \
			Definition64bit		= UE::Trace::Private::FEventInfo::Flag_Definition64, \
			DefinitionBits		= UE::Trace::Private::FEventInfo::DefinitionBits, \
			PartialEventFlags	= (0, ##__VA_ARGS__), \
		}; \
		enum : bool { bIsImportant = ((0, ##__VA_ARGS__) & Important) != 0, bIsDefinition = ((0, ##__VA_ARGS__) & DefinitionBits) != 0,\
		bIsDefinition8 = ((0, ##__VA_ARGS__) & Definition8bit) != 0, \
		bIsDefinition16 = ((0, ##__VA_ARGS__) & Definition16bit) != 0,\
		bIsDefinition32 = ((0, ##__VA_ARGS__) & Definition32bit) != 0, \
		bIsDefinition64 = ((0, ##__VA_ARGS__) & Definition64bit) != 0,}; \
		typedef std::conditional_t<bIsDefinition8, UE::Trace::FEventRef8, std::conditional_t<bIsDefinition16, UE::Trace::FEventRef16 , std::conditional_t<bIsDefinition64, UE::Trace::FEventRef64, UE::Trace::FEventRef32>>> DefinitionType;\
		static constexpr uint32 GetSize() { return EventProps_Meta::Size; } \
		static uint32 GetUid() { static uint32 Uid { Initialize() }; return Uid; } \
		static uint32 FORCENOINLINE Initialize() \
		{ \
			static const uint32 Uid_ThreadSafeInit = [] () \
			{ \
				using namespace UE::Trace; \
				static F##LoggerName##EventName##Fields Fields; \
				static UE::Trace::Private::FEventInfo Info = \
				{ \
					FLiteralName(#LoggerName), \
					FLiteralName(#EventName), \
					(FFieldDesc*)(&Fields), \
					EventProps_Meta::NumFields, \
					uint16(EventFlags), \
				}; \
				return LoggerName##EventName##Event.Initialize(&Info); \
			}(); \
			return Uid_ThreadSafeInit; \
		} \
		typedef UE::Trace::TField<0 /*Index*/, 0 /*Offset*/,

#define TRACE_PRIVATE_EVENT_FIELD(FieldType, FieldName) \
		FieldType> FieldName##_Meta; \
		FieldName##_Meta const FieldName##_Field = UE::Trace::FLiteralName(#FieldName); \
		template <typename... Ts> auto FieldName(Ts... ts) const { \
			LogScopeType::FFieldSet<FieldName##_Meta, FieldType>::Impl((LogScopeType*)this, Forward<Ts>(ts)...); \
			return true; \
		} \
		typedef UE::Trace::TField< \
			FieldName##_Meta::Index + 1, \
			FieldName##_Meta::Offset + FieldName##_Meta::Size,

#define TRACE_PRIVATE_EVENT_REFFIELD(RefLogger, RefEventType, FieldName) \
		F##RefLogger##RefEventType##Fields::DefinitionType> FieldName##_Meta; \
		FieldName##_Meta const FieldName##_Field = FieldName##_Meta(UE::Trace::FLiteralName(#FieldName), RefLogger##RefEventType##Event.GetUid()); \
		template <typename DefinitionType> auto FieldName(UE::Trace::TEventRef<DefinitionType> Reference) const { \
			checkfSlow(Reference.RefTypeId == F##RefLogger##RefEventType##Fields::GetUid(), TEXT("Incorrect reference type passed to event. Field expected %s with uid %u but got a reference with uid %u"), TEXT(#RefEventType), F##RefLogger##RefEventType##Fields::GetUid(), Reference.RefTypeId);\
			LogScopeType::FFieldSet<FieldName##_Meta, F##RefLogger##RefEventType##Fields::DefinitionType>::Impl((LogScopeType*)this, Reference); \
			return true; \
		} \
		typedef UE::Trace::TField< \
			FieldName##_Meta::Index + 1, \
			FieldName##_Meta::Offset + FieldName##_Meta::Size,

#define TRACE_PRIVATE_EVENT_END() \
		std::conditional<bIsDefinition != 0, DefinitionType, UE::Trace::DisabledField>::type> DefinitionId_Meta;\
		DefinitionId_Meta const DefinitionId_Field = UE::Trace::FLiteralName("");\
		static constexpr uint16 NumDefinitionFields = (bIsDefinition != 0) ? 1 : 0;\
		template<typename RefType>\
		auto SetDefinitionId(const RefType& Id) const \
		{ \
			LogScopeType::FFieldSet<DefinitionId_Meta, RefType>::Impl((LogScopeType*)this, Id); \
			return true; \
		} \
		typedef UE::Trace::TField<DefinitionId_Meta::Index + NumDefinitionFields, DefinitionId_Meta::Offset + DefinitionId_Meta::Size, UE::Trace::EventProps> EventProps_Meta; \
		EventProps_Meta const EventProps_Private = {}; \
		static_assert(bIsImportant == 0 || TRACE_PRIVATE_ALLOW_IMPORTANTS, "Important events are disabled in this configuration.");\
		typedef std::conditional<bIsImportant != 0, UE::Trace::Private::FImportantLogScope, UE::Trace::Private::FLogScope>::type LogScopeType; \
		explicit operator bool () const { return true; } \
		enum { EventFlags = PartialEventFlags|((EventProps_Meta::NumAuxFields != 0) ? UE::Trace::Private::FEventInfo::Flag_MaybeHasAux : 0), }; \
		static_assert( \
			(bIsImportant == 0) || (uint32(EventFlags) & uint32(UE::Trace::Private::FEventInfo::Flag_NoSync)), \
			"Trace events flagged as Important events must be marked NoSync" \
		); \
	};

#define TRACE_PRIVATE_LOG_PRELUDE(EnterFunc, LoggerName, EventName, ChannelsExpr, ...) \
	if (TRACE_PRIVATE_CHANNELEXPR_IS_ENABLED(ChannelsExpr)) \
		if (auto LogScope = F##LoggerName##EventName##Fields::LogScopeType::EnterFunc<F##LoggerName##EventName##Fields>(__VA_ARGS__)) \
			if (const auto& __restrict EventName = *UE_LAUNDER((F##LoggerName##EventName##Fields*)(&LogScope))) \
				((void)EventName),

#define TRACE_PRIVATE_LOG_PRELUDE_CONDITIONAL(EnterFunc, LoggerName, EventName, ChannelsExpr, Condition, ...) \
	if (Condition) \
		TRACE_PRIVATE_LOG_PRELUDE(EnterFunc, LoggerName, EventName, ChannelsExpr, __VA_ARGS__) 

#define TRACE_PRIVATE_LOG_EPILOG() \
				LogScope += LogScope

#define TRACE_PRIVATE_LOG(LoggerName, EventName, ChannelsExpr, ...) \
	TRACE_PRIVATE_LOG_PRELUDE(Enter, LoggerName, EventName, ChannelsExpr, ##__VA_ARGS__) \
		TRACE_PRIVATE_LOG_EPILOG()

#define TRACE_PRIVATE_LOG_SCOPED(LoggerName, EventName, ChannelsExpr, ...) \
	UE::Trace::Private::FScopedLogScope UE_JOIN(TheScope, __LINE__); \
	TRACE_PRIVATE_LOG_PRELUDE(ScopedEnter, LoggerName, EventName, ChannelsExpr, ##__VA_ARGS__) \
		UE_JOIN(TheScope, __LINE__).SetActive(), \
		TRACE_PRIVATE_LOG_EPILOG()

#define TRACE_PRIVATE_LOG_SCOPED_CONDITIONAL(LoggerName, EventName, ChannelsExpr, Condition, ...) \
	UE::Trace::Private::FScopedLogScope UE_JOIN(TheScope, __LINE__); \
	TRACE_PRIVATE_LOG_PRELUDE_CONDITIONAL(ScopedEnter, LoggerName, EventName, ChannelsExpr, Condition, ##__VA_ARGS__) \
		UE_JOIN(TheScope, __LINE__).SetActive(), \
		TRACE_PRIVATE_LOG_EPILOG()

#define TRACE_PRIVATE_LOG_SCOPED_T(LoggerName, EventName, ChannelsExpr, ...) \
	UE::Trace::Private::FScopedStampedLogScope UE_JOIN(TheScope, __LINE__); \
	TRACE_PRIVATE_LOG_PRELUDE(ScopedStampedEnter, LoggerName, EventName, ChannelsExpr, ##__VA_ARGS__) \
		UE_JOIN(TheScope, __LINE__).SetActive(), \
		TRACE_PRIVATE_LOG_EPILOG()

#define TRACE_PRIVATE_LOG_SCOPED_T_CONDITIONAL(LoggerName, EventName, ChannelsExpr, Condition, ...) \
	UE::Trace::Private::FScopedStampedLogScope UE_JOIN(TheScope, __LINE__); \
	TRACE_PRIVATE_LOG_PRELUDE_CONDITIONAL(ScopedStampedEnter, LoggerName, EventName, ChannelsExpr, Condition, ##__VA_ARGS__) \
		UE_JOIN(TheScope, __LINE__).SetActive(), \
		TRACE_PRIVATE_LOG_EPILOG()

#define TRACE_PRIVATE_GET_DEFINITION_TYPE_ID(LoggerName, EventName) \
	F##LoggerName##EventName##Fields::GetUid()

#define TRACE_PRIVATE_LOG_DEFINITION(LoggerName, EventName, Id, ChannelsExpr, ...) \
	UE::Trace::MakeEventRef(Id, TRACE_PRIVATE_GET_DEFINITION_TYPE_ID(LoggerName, EventName)); \
	TRACE_PRIVATE_LOG(LoggerName, EventName, ChannelsExpr, ##__VA_ARGS__) \
		<< EventName.SetDefinitionId(UE::Trace::MakeEventRef(Id, F##LoggerName##EventName##Fields::GetUid()))

#endif // TRACE_PRIVATE_MINIMAL_ENABLED

//------------------------------------------------------------------------------------------------
//-- Private no op variants
//------------------------------------------------------------------------------------------------

#define TRACE_PRIVATE_NOP_CHANNEL(ChannelName, ...)

#define TRACE_PRIVATE_NOP_CHANNEL_CUSTOM(ChannelName, ChannelClass, ...)

#define TRACE_PRIVATE_NOP_CHANNEL_EXTERN(ChannelName, ...)

#define TRACE_PRIVATE_NOP_CHANNEL_CUSTOM_EXTERN(ChannelName, ChannelClass, ...)

#define TRACE_PRIVATE_NOP_CHANNEL_DEFINE(ChannelName, ...)

#define TRACE_PRIVATE_NOP_CHANNEL_CUSTOM_DEFINE(ChannelName, ChannelClass, ...)

#define TRACE_PRIVATE_NOP_CHANNELEXPR_IS_ENABLED(ChannelsExpr) \
	false

#define TRACE_PRIVATE_NOP_EVENT_DEFINE(LoggerName, EventName) \
	int8* LoggerName##EventName##DummyPtr = nullptr;

#define TRACE_PRIVATE_NOP_EVENT_BEGIN(LoggerName, EventName, ...) \
	TRACE_PRIVATE_NOP_EVENT_BEGIN_IMPL(LoggerName, EventName)

#define TRACE_PRIVATE_NOP_EVENT_BEGIN_EXTERN(LoggerName, EventName, ...) \
	extern int8* LoggerName##EventName##DummyPtr; \
	TRACE_PRIVATE_NOP_EVENT_BEGIN_IMPL(LoggerName, EventName)

#define TRACE_PRIVATE_NOP_EVENT_BEGIN_IMPL(LoggerName, EventName) \
	struct F##LoggerName##EventName##Dummy \
	{ \
		enum \
		{ \
			Important, \
			NoSync, \
			Definition8bit, \
			Definition16bit, \
			Definition32bit, \
			Definition64bit, \
			DefinitionBits, \
		}; \
		struct FTraceDisabled \
		{ \
			const FTraceDisabled& operator () (...) const { return *this; } \
		}; \
		const F##LoggerName##EventName##Dummy& operator << (const FTraceDisabled&) const \
		{ \
			return *this; \
		} \
		explicit operator bool () const { return false; }

#define TRACE_PRIVATE_NOP_EVENT_FIELD(FieldType, FieldName) \
		const FTraceDisabled& FieldName;

#define TRACE_PRIVATE_NOP_EVENT_REFFIELD(RefLogger, RefEventType, FieldName) \
		const FTraceDisabled& FieldName;

#define TRACE_PRIVATE_NOP_EVENT_END() \
	};

#define TRACE_PRIVATE_NOP_LOG(LoggerName, EventName, ...) \
	if (const auto& EventName = *(F##LoggerName##EventName##Dummy*)1) \
		EventName

#define TRACE_PRIVATE_NOP_LOG_SCOPED(LoggerName, EventName, ...) \
	if (const auto& EventName = *(F##LoggerName##EventName##Dummy*)1) \
		EventName

#define TRACE_PRIVATE_NOP_LOG_SCOPED_CONDITIONAL(LoggerName, EventName, ...) \
	if (const auto& EventName = *(F##LoggerName##EventName##Dummy*)1) \
		EventName

#define TRACE_PRIVATE_NOP_LOG_SCOPED_T(LoggerName, EventName, ...) \
	if (const auto& EventName = *(F##LoggerName##EventName##Dummy*)1) \
		EventName

#define TRACE_PRIVATE_NOP_LOG_SCOPED_T_CONDITIONAL(LoggerName, EventName, ...) \
	if (const auto& EventName = *(F##LoggerName##EventName##Dummy*)1) \
		EventName

#define TRACE_PRIVATE_NOP_GET_DEFINITION_TYPE_ID(LoggerName, EventName) \
	0

#define TRACE_PRIVATE_NOP_LOG_DEFINITION(LoggerName, EventName, Id, ChannelsExpr, ...) \
	UE::Trace::MakeEventRef(Id, 0); \
	TRACE_PRIVATE_NOP_LOG(LoggerName, EventName, ChannelsExpr, ##__VA_ARGS__)
