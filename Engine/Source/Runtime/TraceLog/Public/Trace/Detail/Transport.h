// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

namespace UE {
namespace Trace {

////////////////////////////////////////////////////////////////////////////////
enum ETransport : uint8
{
	_Unused			= 0,
	Raw				= 1,
	Packet			= 2,
	TidPacket		= 3,
	TidPacketSync	= 4,
	Active			= TidPacketSync,
};

////////////////////////////////////////////////////////////////////////////////
enum ETransportTid : uint32
{
	Events		= 0,			// used to describe events
	Internal	= 1,			// events to make the trace stream function
	Importants	= Internal,		// important/cached events
	Bias,						// [Bias,End] = threads. Note bias can't be..
	/* ... */					// ..changed as it breaks backwards compat :(
	End			= 0x3ff0,		// two msbs are user for packet markers
	PseudoImportants	= 0x3ffe,	// see Writer_CallbackOnConnect()
	Sync		= 0x3fff,		// see Writer_SendSync()
};



namespace Private
{

////////////////////////////////////////////////////////////////////////////////
struct FTidPacketBase
{
	enum : uint16
	{
		EncodedMarker = 0x8000,
		PartialMarker = 0x4000, // now unused. fragmented aux-data has an event header
		Verification  = 0x4000, // when set the packet data is immediately followed by 64-bit verification value, see UE_TRACE_PACKET_VERIFICATION
		ThreadIdMask  = PartialMarker - 1,
	};

	uint16 PacketSize;
	uint16 ThreadId;
};

template <uint32 DataSize>
struct TTidPacket
	: public FTidPacketBase
{
	uint8	Data[DataSize];
};

template <uint32 DataSize>
struct TTidPacketEncoded
	: public FTidPacketBase
{
	uint16	DecodedSize;
	uint8	Data[DataSize];
};

using FTidPacket		= TTidPacket<0>;
using FTidPacketEncoded = TTidPacketEncoded<0>;

////////////////////////////////////////////////////////////////////////////////
// Some assumptions are made about 0-sized arrays in the packet structs so we
// will casually make assertions about those assumptions here.
static_assert(sizeof(FTidPacket) == 4, "");
static_assert(sizeof(FTidPacketEncoded) == 6, "");

} // namespace Private

} // namespace Trace
} // namespace UE
