// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if TRACE_PRIVATE_MINIMAL_ENABLED

namespace UE {
namespace Trace {
namespace Private {

////////////////////////////////////////////////////////////////////////////////
/**
 * Structure for managing event data buffers. Most fields are only accessed by the
 * one thread; consuming thread (e.g. worker thread) or producing thread (e.g. any 
 * thread tracing data). Volatile fields are used to safely communicate commited data
 * and buffer usage.
 */
struct FWriteBuffer
{
	void						MoveCursor(uint16 Bytes);
	void						SetCursor(uint8* Ptr);
	uint8*						ReadCursor() const;

	uint8						Overflow[6];
	uint16						CursorMaskInv;
	uint64						PrevTimestamp;
	uint16						Size;
	uint16						ThreadId;
	int32 volatile				EtxOffset; // Shared access (producer and consumer)
	FWriteBuffer* __restrict	NextThread;
	FWriteBuffer* volatile		NextBuffer; //Shared access
	uint8* __restrict volatile	Committed; //Shared access
	uint8* __restrict			Reaped;
	uint8* __restrict			Cursor;
};

////////////////////////////////////////////////////////////////////////////////
inline void FWriteBuffer::MoveCursor(uint16 Bytes)
{
	Cursor += (Bytes & ~CursorMaskInv);
}

////////////////////////////////////////////////////////////////////////////////
inline void FWriteBuffer::SetCursor(uint8* Ptr)
{
	if (CursorMaskInv != 0)
	{
		return;
	}
	Cursor = Ptr;
}

////////////////////////////////////////////////////////////////////////////////
inline uint8* FWriteBuffer::ReadCursor() const
{
	return Cursor;
}

////////////////////////////////////////////////////////////////////////////////
using NextBufferFunc = struct FWriteBuffer*(struct FWriteBuffer*);
struct FTlsState
{
	FWriteBuffer*	WriteBuffer;
	NextBufferFunc*	NextBufferFn;
};

////////////////////////////////////////////////////////////////////////////////
TRACELOG_API uint64				TimeGetTimestamp();
TRACELOG_API FWriteBuffer*		Writer_NextBuffer(FWriteBuffer*);
TRACELOG_API FWriteBuffer*		Writer_GetBuffer();
TRACELOG_API bool				Writer_OnConnectActivate(FTlsState& OutPrevState);
TRACELOG_API void				Writer_OnConnectDeactivate(const FTlsState& PrevState);

////////////////////////////////////////////////////////////////////////////////
#if IS_MONOLITHIC
extern thread_local FTlsState GTlsState;
inline FWriteBuffer* Writer_GetBuffer()
{
	return GTlsState.WriteBuffer;
}
#endif // IS_MONOLITHIC

////////////////////////////////////////////////////////////////////////////////
inline uint64 Writer_GetTimestamp(FWriteBuffer* Buffer)
{
	uint64 Ret = TimeGetTimestamp() - Buffer->PrevTimestamp;
	Buffer->PrevTimestamp += Ret;
	return Ret;
}

} // namespace Private
} // namespace Trace
} // namespace UE

#endif // TRACE_PRIVATE_MINIMAL_ENABLED
