// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Detail/Trace.h" // IWYU pragma: export

////////////////////////////////////////////////////////////////////////////////
#if TRACE_PRIVATE_MINIMAL_ENABLED
#	define UE_TRACE_IMPL(...)
#	define UE_TRACE_API			TRACELOG_API
#else
#	define UE_TRACE_IMPL(...)	{ return __VA_ARGS__; }
#	define UE_TRACE_API			inline
#endif

// msvc seems to have a strange behaviour when it comes to expanding macros.
#if defined(_MSC_VER)

#if TRACE_PRIVATE_FULL_ENABLED
#	define TRACE_IMPL(Macro, ...)		TRACE_PRIVATE_EXPAND(TRACE_PRIVATE_##Macro(__VA_ARGS__))
#else
#	define TRACE_IMPL(Macro, ...)		TRACE_PRIVATE_EXPAND(TRACE_PRIVATE_NOP_##Macro(__VA_ARGS__))
#endif

#if TRACE_PRIVATE_MINIMAL_ENABLED
#	define TRACE_IMPL_MINIMAL(Macro, ...)	TRACE_PRIVATE_EXPAND(TRACE_PRIVATE_##Macro(__VA_ARGS__))
#else
#	define TRACE_IMPL_MINIMAL(Macro, ...)	TRACE_PRIVATE_EXPAND(TRACE_PRIVATE_NOP_##Macro(__VA_ARGS__))
#endif

#else  // defined(_MSC_VER)

#if TRACE_PRIVATE_FULL_ENABLED
#	define TRACE_IMPL(Macro, ...)		TRACE_PRIVATE_EXPAND(TRACE_PRIVATE_##Macro)(__VA_ARGS__)
#else
#	define TRACE_IMPL(Macro, ...)		TRACE_PRIVATE_EXPAND(TRACE_PRIVATE_NOP_##Macro)(__VA_ARGS__)
#endif

#if TRACE_PRIVATE_MINIMAL_ENABLED
#	define TRACE_IMPL_MINIMAL(Macro, ...)	TRACE_PRIVATE_EXPAND(TRACE_PRIVATE_##Macro)(__VA_ARGS__)
#else
#	define TRACE_IMPL_MINIMAL(Macro, ...)	TRACE_PRIVATE_EXPAND(TRACE_PRIVATE_NOP_##Macro)(__VA_ARGS__)
#endif

#endif // defined(_MSC_VER)

////////////////////////////////////////////////////////////////////////////////
namespace UE {
namespace Trace {

// Field types
enum AnsiString {};
enum WideString {};

// Reference to a definition event.
template<typename IdType>
struct TEventRef
{
	using ReferenceType = IdType;

	TEventRef(IdType InId, uint32 InTypeId)
		: Id(InId)
		, RefTypeId(InTypeId)
	{
	}

	IdType Id;
	uint32 RefTypeId;

	uint64 GetHash() const
	{
		if constexpr (std::is_same_v<IdType, uint64>)
		{
			return (uint64(RefTypeId) << 32) ^ Id;
		}
		else
		{
			return (uint64(RefTypeId) << 32) | Id;
		}
	}

private:
	TEventRef() = delete;
};

typedef TEventRef<uint8> FEventRef8;
typedef TEventRef<uint16> FEventRef16;
typedef TEventRef<uint32> FEventRef32;
typedef TEventRef<uint64> FEventRef64;

template<typename IdType>
TEventRef<IdType> MakeEventRef(IdType InId, uint32 InTypeId)
{
	return TEventRef<IdType>(InId, InTypeId);
}

enum class EMessageType : uint8
{
	Reserved			= 0,
	VeryVerbose,
	Verbose,
	// Add to log
	Log,
	// For backwards compatibility
	Info = Log,
	// Display in console or similar
	Display,
	// Warnings to notify user
	WarningStart		= 0x04,
	// Errors are critical to the user, but application
	// can continue to run.
	ErrorStart			= 0x10,
	WriteError,
	ReadError,
	ConnectError,
	ListenError,
	EstablishError,
	FileOpenError,
	WriterError,
	CompressionError,
	PanicError,
	SecureInitError,
	SecureSocketError,
	// Fatal errors should cause application to stop
	FatalStart			= 0x40,
	GenericFatal,
	OOMFatal,
};

struct FMessageEvent
{
	/** Type of message */
	EMessageType		Type;
	/** Type of message stringified */
	const char*			TypeStr;
	/** Clarifying message, may be null for some message types. Pointer only valid during callback. */
	const char*			Description;
};

using OnMessageFunc = void(const FMessageEvent&);
using OnConnectFunc = void(void);
using OnUpdateFunc = void(void);
using OnCloseFunc = void(void);
using OnScopeBeginFunc = void(const ANSICHAR*);
using OnScopeEndFunc = void(void);

struct FInitializeDesc
{
	uint32				TailSizeBytes		= UE_TRACE_DEFAULT_TAIL_SIZE; // set to 0 when minimal tracing
	uint32				ThreadSleepTimeInMS = 0;
	uint32				BlockPoolMaxSize	= UE_TRACE_BLOCK_POOL_MAXSIZE;
	bool				bUseWorkerThread	= true;
	bool				bUseImportantCache	= true;
	uint32				SessionGuid[4]		= {0,0,0,0}; // leave as zero to generate random
	OnConnectFunc*		OnConnectionFunc	= nullptr;
	OnUpdateFunc*		OnUpdateFunc		= nullptr;
	OnScopeBeginFunc*	OnScopeBeginFunc	= nullptr;
	OnScopeEndFunc*		OnScopeEndFunc		= nullptr;
	OnCloseFunc*			OnCloseFunc			= nullptr;
};

typedef uint32 FChannelId;

struct FChannelInfo
{
	const ANSICHAR* Name;
	const ANSICHAR* Desc;
	FChannelId Id;
	bool bIsEnabled;
	bool bIsReadOnly;
};

class FChannel;

/**
 * Allocate memory callback
 * @param Size Size to allocate
 * @param Alignment Alignment of memory
 * @return Pointer to allocated memory
 */
typedef void*		AllocFunc(SIZE_T Size, uint32 Alignment);

/**
 * Free memory callback
 * @param Ptr Memory to free
 * @param Size Size of memory to free
 */
typedef void		FreeFunc(void* Ptr, SIZE_T Size);

/**  
 * The callback provides information about a channel and a user provided pointer.
 * @param Name Name of channel
 * @param State Enabled state of channel
 * @param User User data passed to the callback
 */
typedef void		ChannelIterFunc(const ANSICHAR* Name, bool State, void* User);

/**  
 * The callback provides information about a channel and a user provided pointer.
 * @param Info Information about the channel
 * @param User User data passed to the callback
 * @return Returning false from the callback will stop the enumeration 
 */
typedef bool		ChannelIterCallback(const FChannelInfo& Info, void*/*User*/);

/**
 * User defined write callback.
 * @param Handle User defined handle passed to the function
 * @param Data Pointer to data to write
 * @param Size Size of data to write
 * @return True if all data could be written correctly, false if error occurred.
 */
typedef bool		IoWriteFunc(UPTRINT Handle, const void* Data, uint32 Size);

/**
 * User defined close callback.
 * @param Handle User defined handle passed to the function
 */
typedef void		IoCloseFunc(UPTRINT Handle);	


/**
 * User defined update  callback.
 * @param Handle User defined handle passed to the function
 * @return True while connection is active, false if closed or error state occurred
 */
typedef bool		IoUpdateFunc(UPTRINT Handle);

/**
 * User defined is ready callback. Used to check if a pending connection is ready
 * to be used.
 * @param Handle User defined handle passed to the function
 * @return True if ready, false if further waiting is needed.
 */
typedef bool		IoIsReadyFunc(UPTRINT Handle);

struct FStatistics
{
	uint64	BytesSent				= 0;	// Bytes sent/written to the current endpoint
	uint64	BytesTraced				= 0;	// Bytes sent/written to the current endpoint (uncompressed)
	uint64	BytesEmitted			= 0;	// Bytes emitted but potentially not yet written
	uint64	MemoryUsed				= 0;	// Memory allocated by TraceLog allocator functions
	uint64	BlockPoolAllocated		= 0;	// Memory allocated for the (TLS) block pool.
	uint32  BlockPoolFreeBlocks		= 0;	// Number of available blocks
	uint32	BlockPoolAllocatedBlocks= 0;	// Number of allocated blocks
	uint32	SharedBufferAllocated	= 0;	// Memory allocated for shared buffers
	uint32	FixedBufferAllocated	= 0;	// Memory allocated for fixed buffers (tail, send)
	uint32	CacheAllocated			= 0;	// Total memory allocated in cache buffers
	uint32	CacheUsed				= 0;	// Used cache memory; Important-marked events are stored in the cache.
	uint32	CacheWaste				= 0;	// Unused memory from retired cache buffers
	uint32	ThreadRegistryAllocated = 0;	// Memory allocated for ThreadInfo/GroupBegin event data stored in the ThreadRegistry
};

struct FSendFlags
{
	static const uint16 None		= 0;
	static const uint16 ExcludeTail	= 1 << 0;	// do not send the tail of historical events
	static const uint16 _Reserved	= 1 << 15;	// this bit is used internally
};

/**
 * Set optional allocation and free methods to use. If not set TraceLog will fall back to
 * default platform allocation methods.
 * @param Alloc Callback for allocations
 * @param Free Callback for free
 */
UE_TRACE_API void	SetMemoryHooks(AllocFunc Alloc, FreeFunc Free) UE_TRACE_IMPL();

/**
 * Set optional callback to use for critical messages. See \ref OnMessageFunc for details.
 * @param MessageFunc Function to call for critical messages
 */
UE_TRACE_API void	SetMessageCallback(OnMessageFunc* MessageFunc) UE_TRACE_IMPL();

/**
 * Set optional on update callbacks. If set issued after every update and once after initialization.
 * @param UpdateFunc Function to call after every update
 */
UE_TRACE_API void	SetUpdateCallback(OnUpdateFunc* UpdateFunc) UE_TRACE_IMPL();

/**
 * Initialize TraceLog library.
 * @param Desc Initialization options
 */
UE_TRACE_API void	Initialize(const FInitializeDesc& Desc) UE_TRACE_IMPL();

/**
 * Manually start worker thread if library is initialized without worker thread.
 */
UE_TRACE_API void	StartWorkerThread() UE_TRACE_IMPL();

/**
 * Call when application is exiting. Notifies TraceLog that the worker thread can 
 * disappear at any time and clears block pool limits. Tracing is still possible.
 */
UE_TRACE_API void	Exit() UE_TRACE_IMPL();

/**
 * Shuts down the library completely and frees resources. After this tracing will 
 * not be possible.
 */
UE_TRACE_API void	Shutdown() UE_TRACE_IMPL();

/**
 * Puts TraceLog in a critical failure stats. Disables all tracing by muting all channels.
 * Tracing will be totally disabled for the remainder of the process lifetime.
 */
UE_TRACE_API void	Panic() UE_TRACE_IMPL();

/**
 * Manually update TraceLog if no worker thread is running. Only one thread (including the 
 * worker thread) is able to enter the update method at any point.
 */
UE_TRACE_API void	Update() UE_TRACE_IMPL();

/**
 * Fetches tracked telemetry from the library.
 * @param Out Pointer to struct where telemetry values will be written to.
 */ 
UE_TRACE_API void	GetStatistics(FStatistics& Out) UE_TRACE_IMPL();

/**
 * Setup TraceLog to output to remote host using a socket connection, to take effect next update. 
 * Will fail if another pending output has been queued or the host is unreachable.
 * @param Host Target hostname or ip
 * @param Port Target port
 * @param Flags Options for the connection
 * @return True on successful connection to host, false otherwise
 */ 
UE_TRACE_API bool	SendTo(const TCHAR* Host, uint32 Port=0, uint16 Flags=FSendFlags::None) UE_TRACE_IMPL(false);

/**
 * Setup TraceLog to output to a new or existing file, to take effect next update. Will fail if another pending 
 * output has been queued or if the file location is not writeable.
 * @param Path Target path
 * @param Flags Options for the connection
 * @return True if the file could be opened or created correctly, false otherwise
 */
UE_TRACE_API bool	WriteTo(const TCHAR* Path, uint16 Flags=FSendFlags::None) UE_TRACE_IMPL(false);

/**
 * Setup TraceLog to output to user defined callback, to take effect next update. Will fail if another pending
 * output has been queued.
 * @param WriteFunc Function to call when writing data
 * @param CloseFunc Function to call when output is closed
 * @param Flags Options for the connection
 * @return True if successful, false otherwise
 */
UE_TRACE_API bool	RelayTo(UPTRINT InHandle, IoWriteFunc WriteFunc, IoCloseFunc CloseFunc, uint16 Flags = FSendFlags::None) UE_TRACE_IMPL(false);

/**
 * Immediately write contents of tail buffers and important events to a new or existing file.
 * @param Path Target path
 * @param MaxTailSize Maximum size of tail buffer to write. Zero indicates no limit.
 */
UE_TRACE_API bool	WriteSnapshotTo(const TCHAR* Path, uint32 MaxTailSize = 0) UE_TRACE_IMPL(false);

/**
 * Immediately write contents of tail buffers and important events to a remote host.
 * @param Host Target hostname or ip
 * @param Port Target port
 * @param MaxTailSize Maximum size of tail buffer to write. Zero indicates no limit.
 */
UE_TRACE_API bool	SendSnapshotTo(const TCHAR* Host, uint32 Port, uint32 MaxTailSize = 0) UE_TRACE_IMPL(false);

/**
 * Checks if TraceLog currently has an output. Note that trace events can still be recorded
 * and saved in tail buffers regardless if an output is active.
 * @return True if an output is active, false if not
 */
UE_TRACE_API bool	IsTracing() UE_TRACE_IMPL(false);

/**
 * Checks if TraceLog currently has an output and return session and trace GUIDs of active trace stream. 
 * Note that trace events can still be recorded and saved in tail buffers regardless if an output is active.
 * @param OutSessionGuid If output is active current session GUID will be written, otherwise it will be unchanged
 * @param OutTraceGuid If output is active current trace GUID will be written, otherwise it will be unchanged
 * @return True if an output is active, false if not
 */
UE_TRACE_API bool	IsTracingTo(uint32 (&OutSessionGuid)[4], uint32 (&OutTraceGuid)[4]) UE_TRACE_IMPL(false);

/**
* Checks if TraceLog is currently using the Tail Buffer, which means that trace events are being cached in memory and
* could be dumped into a file at any time.
* @return True if the TailBuffer is active, false if not
*/
UE_TRACE_API bool	IsTailing() UE_TRACE_IMPL(false);

/**
 * Stops current output if any is active.This will fail there is no active output or if there is already a queued
 * output to be started.
 * @return True if any output was stopped, false if no output was active or there is a queued output to be started.
 */
UE_TRACE_API bool	Stop() UE_TRACE_IMPL(false);

/**
 * Checks if a string is valid channel name.
 * @param ChannelName String to check
 * @return True if there is a registered channel object with the same name.
 */
UE_TRACE_API bool	IsChannel(const TCHAR* ChannelName) UE_TRACE_IMPL(false);

/**
 * Toggles channel to control output of events.
 * @param ChannelName Name of channel to control
 * @param bEnabled Set if channel should be enabled or disabled
 * @return Final channel state. If the channel name was not found return false.
 */
UE_TRACE_API bool	ToggleChannel(const TCHAR* ChannelName, bool bEnabled) UE_TRACE_IMPL(false);

/**
 * Enumerates registered channels.
 * @param IterFunc Function to call for each channel
 * @param User Optional pointer to user data to pass to callback
 */
UE_TRACE_API void	EnumerateChannels(ChannelIterFunc IterFunc, void* User) UE_TRACE_IMPL();

/**
 * Enumerates registered channels.
 * @param IterFunc Function to call for each channel
 * @param User Optional pointer to user data to pass to callback
 */
UE_TRACE_API void	EnumerateChannels(ChannelIterCallback IterFunc, void* User) UE_TRACE_IMPL();

/**
 * Register a new thread in Trace. This is a requirement before tracing anything from the thread.
 * @param Name Display name of thread
 * @param SystemId Platform specific thread id
 * @param SortHint Suggestion on how to order thread in presentation
 */
UE_TRACE_API void	ThreadRegister(const TCHAR* Name, uint32 SystemId, int32 SortHint) UE_TRACE_IMPL();

/**
 * Define a group of threads with similar use. Any thread created within this call and /ref ThreadGroupEnd
 * will be grouped together.
 * @param Name Display name of group
 */
UE_TRACE_API void	ThreadGroupBegin(const TCHAR* Name) UE_TRACE_IMPL();

/**
 * End a group of threads with similar use. See /ref ThreadGroupBegin. 
 */
UE_TRACE_API void	ThreadGroupEnd() UE_TRACE_IMPL();

/**
 * Attempts to find the corresponding channel object given a channel name.
 * @param ChannelName Name to search for
 * @return Pointer to channel object, or null if no channel matching the name have been registered
 */
UE_TRACE_API FChannel* FindChannel(const TCHAR* ChannelName) UE_TRACE_IMPL(nullptr);

/**
 * Attempts to find the corresponding channel object given a channel id.
 * @param ChannelId Id to search for
 * @return Pointer to channel object, or null if no channel with matching id have been registered
 */
UE_TRACE_API FChannel* FindChannel(FChannelId ChannelId) UE_TRACE_IMPL(nullptr);


#if TRACE_PRIVATE_ALLOW_SECURE_TRACING


enum class ESecureTraceMode : uint8
{
	SelfSigned,
	CertificateFile,
	PreSharedKeys,
};

/**
 * Options for initializing secure tracing. Supports three modes of operation 
 * depending on what values are set.
 * * External certificate + private key and optional authentication token.
 * * Self signed certificate and optional authentication token.
 * * Pre-shared keys
 */
struct FSecureTraceSettings 
{
	/**
	 * Desired operational mode.
	 */
	ESecureTraceMode Mode = ESecureTraceMode::SelfSigned;

	/**
	 * Port used to listen for peer connections.
	 */
	uint16 Port = 0;

	/**
	 * Path to certificate file. Only used if Mode is CertificateFile.
	 */
	const ANSICHAR* CertificatePath = nullptr;

	/**
	 * Path to private key. Only used if Mode is CertificateFile.
	 */
	const ANSICHAR* PrivateKeyPath = nullptr;
};

/**
 * Initialize secure connection component. This must happen exactly once before
 * it's possible to call \ref SecureSendTo.
 * @param Settings Settings to use for secure connections
 */
UE_TRACE_API bool SecureInitialize(const FSecureTraceSettings& Settings);

/**
 * Checks if secure connection component has been initialized.
 * @return true if \ref SecureInitialize has already been called.
 */
UE_TRACE_API bool SecureIsInitialized();

/**
 * Retrieve the public certificate (DER format) when using a self-signed certificate.
 * @param OutPtr If successful, the pointer to the certificate data. The pointer is valid for the remiander of the process lifetime.
 * @param OutSize If successful, certificate size in bytes.
 * @return True if initialized with self-signed certificate, and the certificate was successfully generated.
 */
UE_TRACE_API bool SecureGetSelfSignedCertificate(uint8** OutPtr, uint32* OutSize);

/**
 * Options for a secure trace connection. Allows to set a unique authorization token 
 * and identity where applicable.
 */
struct FSecureTraceOptions
{
	/**
	 * AuthorizationToken that peer will use to connect. Used as pre-shared keys if Mode is PreSharedKeys
	 */
	const uint8* AuthorizationToken = nullptr;

	/**
	 * Identity string expected. Used when Mode is PreSharedKeys. String does not need to outlive function call.
	 */
	const char* Identity = nullptr;

	/**
	 * Expected authorization token size in bytes. Must be set if AuthorizationToken is set.
	 */
	uint32 AuthorizationTokenSize = 0u;

	/**
	 * If the connection can receive data.
	 */
	bool bCanReceive = false;

	/**
	 * Timeout in ms for connect and handshakes. If peer doesn't connect
	 * or supply handshake within this time connection is closed. Zero is
	 * interpreted as no timeouts should be applied.
	 */
	uint32 TimeoutMs = 0u;
};


/**
 * Setup output to a secure socket connection. This requires the receiver to connect to the tracing machine on the
 * specified listening port. Additionally certificates to use and optional authorization tokens can be specified for
 * the connection.
 * @param Options Configuration options for the connection
 * @param Flags Options for the connection
 */
UE_TRACE_API bool	SecureSendTo(const FSecureTraceOptions& Options, uint16 Flags = FSendFlags::None);


#endif // TRACE_PRIVATE_ALLOW_SECURE_TRACING

} // namespace Trace
} // namespace UE


////////////////////////////////////////////////////////////////////////////////
/// Tracing macros
/// Use these to define event types, channel and emit events.
////////////////////////////////////////////////////////////////////////////////
#define UE_TRACE_EVENT_DEFINE(LoggerName, EventName)											TRACE_IMPL(EVENT_DEFINE, LoggerName, EventName)
#define UE_TRACE_EVENT_BEGIN(LoggerName, EventName, ...)										TRACE_IMPL(EVENT_BEGIN, LoggerName, EventName, ##__VA_ARGS__)
#define UE_TRACE_EVENT_BEGIN_EXTERN(LoggerName, EventName, ...)									TRACE_IMPL(EVENT_BEGIN_EXTERN, LoggerName, EventName, ##__VA_ARGS__)
#define UE_TRACE_EVENT_FIELD(FieldType, FieldName)												TRACE_IMPL(EVENT_FIELD, FieldType, FieldName)
#define UE_TRACE_EVENT_REFERENCE_FIELD(RefLogger, RefEvent, FieldName)							TRACE_IMPL(EVENT_REFFIELD, RefLogger, RefEvent, FieldName)
#define UE_TRACE_EVENT_END()																	TRACE_IMPL(EVENT_END)
#define UE_TRACE_LOG(LoggerName, EventName, ChannelsExpr, ...)									TRACE_IMPL(LOG, LoggerName, EventName, ChannelsExpr, ##__VA_ARGS__)
#define UE_TRACE_LOG_SCOPED(LoggerName, EventName, ChannelsExpr, ...)							TRACE_IMPL(LOG_SCOPED, LoggerName, EventName, ChannelsExpr, ##__VA_ARGS__)
#define UE_TRACE_LOG_SCOPED_CONDITIONAL(LoggerName, EventName, ChannelsExpr, Condition, ...)	TRACE_IMPL(LOG_SCOPED_CONDITIONAL, LoggerName, EventName, ChannelsExpr, Condition, ##__VA_ARGS__)
#define UE_TRACE_LOG_SCOPED_T(LoggerName, EventName, ChannelsExpr, ...)							TRACE_IMPL(LOG_SCOPED_T, LoggerName, EventName, ChannelsExpr, ##__VA_ARGS__)
#define UE_TRACE_LOG_SCOPED_T_CONDITIONAL(LoggerName, EventName, ChannelsExpr, Condition, ...)	TRACE_IMPL(LOG_SCOPED_T_CONDITIONAL, LoggerName, EventName, ChannelsExpr, Condition, ##__VA_ARGS__)
#define UE_TRACE_GET_DEFINITION_TYPE_ID(LoggerName, EventName)									TRACE_IMPL(GET_DEFINITION_TYPE_ID, LoggerName, EventName)
#define UE_TRACE_LOG_DEFINITION(LoggerName, EventName, Id, ChannelsExpr, ...)					TRACE_IMPL(LOG_DEFINITION, LoggerName, EventName, Id, ChannelsExpr, ##__VA_ARGS__)
#define UE_TRACE_CHANNEL(ChannelName, ...)														TRACE_IMPL(CHANNEL, ChannelName, ##__VA_ARGS__)
#define UE_TRACE_CHANNEL_CUSTOM(ChannelName, ChannelClass, ...)									TRACE_IMPL(CHANNEL_CUSTOM, ChannelName, ChannelClass, ##__VA_ARGS__)
#define UE_TRACE_CHANNEL_EXTERN(ChannelName, ...)												TRACE_IMPL(CHANNEL_EXTERN, ChannelName, ##__VA_ARGS__)
#define UE_TRACE_CHANNEL_CUSTOM_EXTERN(ChannelName, ChannelClass, ...)							TRACE_IMPL(CHANNEL_CUSTOM_EXTERN, ChannelName, ChannelClass, ##__VA_ARGS__)
#define UE_TRACE_CHANNEL_DEFINE(ChannelName, ...)												TRACE_IMPL(CHANNEL_DEFINE, ChannelName, ##__VA_ARGS__)
#define UE_TRACE_CHANNEL_CUSTOM_DEFINE(ChannelName, ChannelClass, ...)							TRACE_IMPL(CHANNEL_CUSTOM_DEFINE, ChannelName, ChannelClass, ##__VA_ARGS__)
#define UE_TRACE_CHANNELEXPR_IS_ENABLED(ChannelsExpr)											TRACE_IMPL(CHANNELEXPR_IS_ENABLED, ChannelsExpr)


////////////////////////////////////////////////////////////////////////////////
/// Shipping variants of the macros.
/// With these macros users can provide a subset of events that are available
/// both in development and in shipping configurations (provided UE_TRACE_MINIMAL_ENABLED is set).
////////////////////////////////////////////////////////////////////////////////
#define UE_TRACE_MINIMAL_EVENT_DEFINE(LoggerName, EventName)											TRACE_IMPL_MINIMAL(EVENT_DEFINE, LoggerName, EventName)
#define UE_TRACE_MINIMAL_EVENT_BEGIN(LoggerName, EventName, ...)										TRACE_IMPL_MINIMAL(EVENT_BEGIN, LoggerName, EventName, ##__VA_ARGS__)
#define UE_TRACE_MINIMAL_EVENT_BEGIN_EXTERN(LoggerName, EventName, ...)									TRACE_IMPL_MINIMAL(EVENT_BEGIN_EXTERN, LoggerName, EventName, ##__VA_ARGS__)
#define UE_TRACE_MINIMAL_EVENT_FIELD(FieldType, FieldName)												TRACE_IMPL_MINIMAL(EVENT_FIELD, FieldType, FieldName)
#define UE_TRACE_MINIMAL_EVENT_REFERENCE_FIELD(RefLogger, RefEvent, FieldName)							TRACE_IMPL_MINIMAL(EVENT_REFFIELD, RefLogger, RefEvent, FieldName)
#define UE_TRACE_MINIMAL_EVENT_END()																	TRACE_IMPL_MINIMAL(EVENT_END)
#define UE_TRACE_MINIMAL_LOG(LoggerName, EventName, ChannelsExpr, ...)									TRACE_IMPL_MINIMAL(LOG, LoggerName, EventName, ChannelsExpr, ##__VA_ARGS__)
#define UE_TRACE_MINIMAL_LOG_SCOPED(LoggerName, EventName, ChannelsExpr, ...)							TRACE_IMPL_MINIMAL(LOG_SCOPED, LoggerName, EventName, ChannelsExpr, ##__VA_ARGS__)
#define UE_TRACE_MINIMAL_LOG_SCOPED_CONDITIONAL(LoggerName, EventName, ChannelsExpr, Condition, ...)	TRACE_IMPL_MINIMAL(LOG_SCOPED_CONDITIONAL, LoggerName, EventName, ChannelsExpr, Condition, ##__VA_ARGS__)
#define UE_TRACE_MINIMAL_LOG_SCOPED_T(LoggerName, EventName, ChannelsExpr, ...)							TRACE_IMPL_MINIMAL(LOG_SCOPED_T, LoggerName, EventName, ChannelsExpr, ##__VA_ARGS__)
#define UE_TRACE_MINIMAL_LOG_SCOPED_T_CONDITIONAL(LoggerName, EventName, ChannelsExpr, Condition, ...)	TRACE_IMPL_MINIMAL(LOG_SCOPED_T_CONDITIONAL, LoggerName, EventName, ChannelsExpr, Condition, ##__VA_ARGS__)
#define UE_TRACE_MINIMAL_GET_DEFINITION_TYPE_ID(LoggerName, EventName)									TRACE_IMPL_MINIMAL(GET_DEFINITION_TYPE_ID, LoggerName, EventName)
#define UE_TRACE_MINIMAL_LOG_DEFINITION(LoggerName, EventName, Id, ChannelsExpr, ...)					TRACE_IMPL_MINIMAL(LOG_DEFINITION, LoggerName, EventName, Id, ChannelsExpr, ##__VA_ARGS__)
#define UE_TRACE_MINIMAL_CHANNEL(ChannelName, ...)														TRACE_IMPL_MINIMAL(CHANNEL, ChannelName, ##__VA_ARGS__)
#define UE_TRACE_MINIMAL_CHANNEL_CUSTOM(ChannelName, ChannelClass, ...)									TRACE_IMPL_MINIMAL(CHANNEL_CUSTOM, ChannelName, ChannelClass, ##__VA_ARGS__)
#define UE_TRACE_MINIMAL_CHANNEL_EXTERN(ChannelName, ...)												TRACE_IMPL_MINIMAL(CHANNEL_EXTERN, ChannelName, ##__VA_ARGS__)
#define UE_TRACE_MINIMAL_CHANNEL_CUSTOM_EXTERN(ChannelName, ChannelClass, ...)							TRACE_IMPL_MINIMAL(CHANNEL_CUSTOM_EXTERN, ChannelName, ChannelClass, ##__VA_ARGS__)
#define UE_TRACE_MINIMAL_CHANNEL_DEFINE(ChannelName, ...)												TRACE_IMPL_MINIMAL(CHANNEL_DEFINE, ChannelName, ##__VA_ARGS__)
#define UE_TRACE_MINIMAL_CHANNEL_CUSTOM_DEFINE(ChannelName, ChannelClass, ...)							TRACE_IMPL_MINIMAL(CHANNEL_CUSTOM_DEFINE, ChannelName, ChannelClass, ##__VA_ARGS__)
#define UE_TRACE_MINIMAL_CHANNELEXPR_IS_ENABLED(ChannelsExpr)											TRACE_IMPL_MINIMAL(CHANNELEXPR_IS_ENABLED, ChannelsExpr)
