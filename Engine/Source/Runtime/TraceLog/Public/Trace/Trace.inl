// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// IWYU pragma: begin_exports
#include "CoreTypes.h"
#include "Trace/Detail/Channel.inl"
#include "Trace/Detail/EventNode.h"
#include "Trace/Detail/Important/ImportantLogScope.inl"
#include "Trace/Detail/LogScope.inl"
#include "Trace/Detail/Writer.inl"
// IWYU pragma: end_exports
