// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System;
using System.IO;

public class TraceLog : ModuleRules
{
	public TraceLog(ReadOnlyTargetRules Target) : base(Target)
	{
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;

		bRequiresImplementModule = false;

		PublicIncludePathModuleNames.Add("Core");

		if (Target.bBuildEditor)
		{
			bDisableAutoRTFMInstrumentation = true;
		}

		PrivateDefinitions.Add("SUPPRESS_PER_MODULE_INLINE_FILE"); // This module does not use core's standard operator new/delete overloads
		PrivateDefinitions.Add("UE_TRACE_SUPPRESS_ASSERT"); //No assert in TraceLog module itself

		if (Target.bEnableTraceSecureConnection)
		{
			PublicDefinitions.Add("UE_TRACE_ALLOW_SECURE_TRACING=1");
			PrivateDependencyModuleNames.Add("OpenSSL");
		}

		if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows))
		{
			PrivateDefinitions.Add("_CRT_SECURE_NO_WARNINGS");
		}
	}
}
