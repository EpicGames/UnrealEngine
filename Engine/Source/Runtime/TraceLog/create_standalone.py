import os
import re
import stat
import shutil
import argparse
import subprocess
from pathlib import Path

#-------------------------------------------------------------------------------
def _spam_header(*args, **kwargs):
	_spam("\x1b[96m##", *args, "\x1b[0m", **kwargs)

def _spam(*args, **kwargs):
	print(*args, **kwargs)

#-------------------------------------------------------------------------------
class _SourceFile(object):
	def __init__(self, path):
		self.path = path
		self.is_include = (path.suffix == ".h")
		self.lines = []
		self.deps = []
		self.ext_deps = []

	def __eq__(self, rhs):
		return rhs.samefile(self.path)

	def __hash__(self):
		return hash(self.path)

#-------------------------------------------------------------------------------
def _parse_include(line):
	m = re.match(r'\s*#\s*include\s+\"([^"]+)\"', line)
	return None if not m else Path(m.group(1))

def _exclude_line(line):
	line = line.strip()

	if "#pragma once" in line:	return True
	if line.startswith("//"):	return True
	if not line:				return True

#-------------------------------------------------------------------------------
def _collect_source(src_dir, predicate=None):
	files  = [x for x in src_dir.glob("Public/**/*")]
	files += [x for x in src_dir.glob("Private/**/*")]
	files  = [x for x in files if x.suffix in (".h", ".cpp", ".inl")]
	_spam("Found %d files" % len(files))

	if predicate:
		files = [x for x in files if predicate(x)]
		_spam("Filtered down to %d files" % len(files))

	return [_SourceFile(x) for x in files]

#-------------------------------------------------------------------------------
def _finalize_source(source_files, include_dirs):
	# Collect lines of each source file, filtering local includes
	_spam("Loading code")
	for source_file in source_files:
		file = source_file.path
		for line in file.open("rt", encoding="utf-8-sig"):
			include = _parse_include(line)
			if not include:
				if not _exclude_line(line):
					source_file.lines.append(line)
				continue

			for include_dir in (file.parent, *include_dirs):
				candidate = include_dir / include
				if candidate.is_file():
					source_file.deps.append(candidate)
					break
			else:
				source_file.ext_deps.append(include)

	# Hook up dependencies
	_spam("Resolving include dependencies")
	for source_file in source_files:
		new_deps = []
		for dep in source_file.deps:
			try: index = source_files.index(dep)
			except ValueError: pass
			dep = source_files[index]
			new_deps.append(dep)
		source_file.deps = new_deps

	# Topologically sort by dependencies
	_spam("Sorting")
	visited = set()
	topo_sorted = []
	def visit(source_file):
		if source_file in visited:
			return

		for x in source_file.deps:
			visit(x)

		topo_sorted.append(source_file)
		visited.add(source_file)

	for x in source_files:
		visit(x)

	# Stable sort to put .cpps last
	ext_key = lambda x: 1 if x.path.suffix == ".cpp" else 0
	source_files = sorted(topo_sorted, key=ext_key)

	return source_files

#-------------------------------------------------------------------------------
def _main_trace(src_dir, dest_dir, thin, analysis):
	dest_dir = dest_dir.resolve()
	src_dir = Path(__file__).parent

	_spam("Source dir:", src_dir)
	_spam("Dest dir:", dest_dir)

	# Collect and sort source files.
	def predicate(path):
		if path.stem.startswith("lz4"): return False
		return True
	source_files = _collect_source(src_dir, predicate)

	analysis_dir = None
	if analysis:
		analysis_dir = Path(__file__).parent
		analysis_dir = analysis_dir.parent.parent / "Developer/TraceAnalysis"
		_spam("Analysis dir:", analysis_dir)

		def predicate(path):
			if	 path.parent.name == "Store":		 return False
			elif path.parent.name == "Asio":		 return False
			elif path.name.startswith("Store"):		 return False
			elif path.name.startswith("Control"):	 return False
			excludes = (
				"Analysis.h",
				"Context.cpp",
				"DataStream.h",
				"DataStream.cpp",
				"EventToCbor.cpp",
				"Processor.cpp",
				"Processor.h",
				"TraceAnalysisModule.cpp",
			)
			return path.name not in excludes
		source_files += _collect_source(analysis_dir, predicate)

	_spam_header("Topological sort")
	include_dirs = (src_dir / "Public",)
	if analysis_dir:
		include_dirs = (*include_dirs, analysis_dir / "Public")
	source_files = _finalize_source(source_files, include_dirs)

	# Add prologue and epilogue files
	prologue = _SourceFile(src_dir / "standalone_prologue.h.src")
	source_files.insert(0, prologue)
	prologue.lines = [x for x in prologue.path.open("rt") if x]

	epilogue = _SourceFile(src_dir / "standalone_epilogue.h.src")
	source_files.append(epilogue)
	epilogue.lines = [x for x in epilogue.path.open("rt") if x]

	# Write the output file
	_spam_header("Output")
	_spam(dest_dir / "trace.h")
	with (dest_dir / "trace.h").open("wt") as out:
		print("// Copyright Epic Games, Inc. All Rights Reserved.", file=out)
		print("#pragma once", file=out)

		if analysis_dir:
			print("#define TRACE_HAS_ANALYSIS", file=out)

		class State(object): pass
		state = State()
		state.out = out
		state.dest_dir = dest_dir
		state.src_dir = src_dir
		state.analysis_dir = analysis_dir
		state.source_files = source_files
		return _thin(state) if thin else _fat(state)

	_spam("...done!")

#-------------------------------------------------------------------------------
def _fat(state):
	out = state.out
	cpp_started = False
	for source_file in state.source_files:
		if not cpp_started:
			if source_file.path.suffix == ".cpp":
				print("#if TRACE_IMPLEMENT", file=out)
				cpp_started = True
		elif source_file.path.suffix != ".cpp":
			print("#endif // TRACE_IMPLEMENT", file=out)
			cpp_started = False

		print("/* {{{1", source_file.path.name, "*/", file=out)
		print(file=out)
		for line in source_file.lines:
			out.write(line)

#-------------------------------------------------------------------------------
def _thin(state):
	# Include trace source code
	def write_include(source_file):
		path = str(source_file.path).replace("\\", "/")
		print(fr'#include "{path}"', file=state.out)

	out = state.out
	write_include(state.source_files[0])
	print("#if TRACE_IMPLEMENT", file=out)
	for source_file in state.source_files[1:-1]:
		if source_file.path.suffix == ".cpp":
			write_include(source_file)
	print("#endif // TRACE_IMPLEMENT", file=out)
	write_include(state.source_files[-1])

	# Stub out external includes
	ext_deps = set()
	for source_file in state.source_files:
		for dep in source_file.ext_deps:
			ext_deps.add(dep)

	stubs_dir = state.dest_dir / "include"
	for dep in ext_deps:
		(stubs_dir / dep).parent.mkdir(parents=True, exist_ok=True)
		(stubs_dir / dep).open("wt").close()

	# Symlink source
	def symlink_posix(fr, to):
		fr = state.dest_dir / "src" / fr
		fr.symlink_to(to.resolve())

	def symlink_nt(fr, to):
		fr = state.dest_dir / "src" / fr
		subprocess.run(
			("cmd.exe", "/c", "mklink", "/j", fr, to.resolve()),
			stderr=subprocess.DEVNULL
		)

	symlink = symlink_nt if os.name == "nt" else symlink_posix

	(state.dest_dir / "src").mkdir(parents=True, exist_ok=True)
	symlink("trace", state.src_dir)
	if state.analysis_dir:
		symlink("analysis", state.analysis_dir)

#-------------------------------------------------------------------------------
def main():
	desc = "Amalgamate TraceLog into a standalone single-file library"
	parser = argparse.ArgumentParser(description=desc)
	parser.add_argument("outdir", help="Directory to write output file(s) to")
	parser.add_argument("--thin", action="store_true", help="#include everything instead of blitting")
	parser.add_argument("--analysis", action="store_true", help="Amalgamate TraceAnalysis too")
	args = parser.parse_args()

	_main_trace(Path(__file__).parent, Path(args.outdir), args.thin, args.analysis)

if __name__ == "__main__":
	raise SystemExit(main())

# vim: noexpandtab
