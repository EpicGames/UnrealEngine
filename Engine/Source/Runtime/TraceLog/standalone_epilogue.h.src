// Copyright Epic Games, Inc. All Rights Reserved.

// HEADER_UNIT_SKIP - Not included directly

// {{{1 amalgamation-tail //////////////////////////////////////////////////////

#if PLATFORM_WINDOWS
#	pragma warning(pop)
#endif

#if TRACE_UE_COMPAT_LAYER

namespace trace {

#if defined(TRACE_HAS_ANALYSIS)

inline void SerializeToCborImpl(TArray<uint8>&, const IAnalyzer::FEventData&, uint32)
{
	*(int32*)0 = 0; // unsupported
}

#endif // TRACE_HAS_ANALYSIS

} // namespace trace

#if PLATFORM_WINDOWS
#	if defined(UNICODE) || defined(_UNICODE)
#		undef TEXT
#		undef TCHAR
#		define TEXT(x)	L##x
#	endif
#endif

#undef check

#endif // TRACE_UE_COMPAT_LAYER



// {{{1 library-setup //////////////////////////////////////////////////////////

#include <string_view>

#define TRACE_EVENT_DEFINE			UE_TRACE_EVENT_DEFINE
#define TRACE_EVENT_BEGIN			UE_TRACE_EVENT_BEGIN
#define TRACE_EVENT_BEGIN_EXTERN	UE_TRACE_EVENT_BEGIN_EXTERN
#define TRACE_EVENT_FIELD			UE_TRACE_EVENT_FIELD
#define TRACE_EVENT_END				UE_TRACE_EVENT_END
#define TRACE_LOG					UE_TRACE_LOG
#define TRACE_LOG_SCOPED			UE_TRACE_LOG_SCOPED
#define TRACE_LOG_SCOPED_T			UE_TRACE_LOG_SCOPED_T
#define TRACE_CHANNEL				UE_TRACE_CHANNEL
#define TRACE_CHANNEL_EXTERN		UE_TRACE_CHANNEL_EXTERN
#define TRACE_CHANNEL_DEFINE		UE_TRACE_CHANNEL_DEFINE

namespace trace {
	using namespace UE::Trace;
	namespace detail {
		using namespace UE::Trace::Private;
	}
}

#define TRACE_PRIVATE_CONCAT_(x, y)		x##y
#define TRACE_PRIVATE_CONCAT(x, y)		TRACE_PRIVATE_CONCAT_(x, y)
#define TRACE_PRIVATE_UNIQUE_VAR(name)	TRACE_PRIVATE_CONCAT($trace_##name, __LINE__)



// {{{1 session-header /////////////////////////////////////////////////////////

namespace trace {

enum class Build
{
	Unknown,
	Debug,
	DebugGame,
	Development,
	Shipping,
	Test
};

void DescribeSession(
	const std::string_view& AppName,
	Build Variant=Build::Unknown,
	const std::string_view& CommandLine="",
	const std::string_view& BuildVersion="unknown_ver");

} // namespace trace

// {{{1 session-source /////////////////////////////////////////////////////////

#if TRACE_IMPLEMENT

namespace trace {
namespace detail {

TRACE_EVENT_BEGIN(Diagnostics, Session2, NoSync|Important)
	TRACE_EVENT_FIELD(uint8, ConfigurationType)
	TRACE_EVENT_FIELD(trace::AnsiString, AppName)
	TRACE_EVENT_FIELD(trace::AnsiString, BuildVersion)
	TRACE_EVENT_FIELD(trace::AnsiString, Platform)
	TRACE_EVENT_FIELD(trace::AnsiString, CommandLine)
TRACE_EVENT_END()

} // namespace detail

////////////////////////////////////////////////////////////////////////////////
void DescribeSession(
	const std::string_view& AppName,
	Build Variant,
	const std::string_view& CommandLine,
	const std::string_view& BuildVersion)
{
	using namespace detail;
	using namespace std::literals;

	std::string_view Platform;
#if PLATFORM_WINDOWS
	Platform = "Windows"sv;
#elif PLATFORM_UNIX
	Platform = "Linux"sv;
#elif PLATFORM_MAC
	Platform = "Mac"sv;
#else
	Platform = "Unknown"sv;
#endif

	int32 DataSize = 0;
	DataSize += int32(AppName.size());
	DataSize += int32(BuildVersion.size());
	DataSize += int32(Platform.size());
	DataSize += int32(CommandLine.size());

	TRACE_LOG(Diagnostics, Session2, true, DataSize)
		<< Session2.AppName(AppName.data(), int32(AppName.size()))
		<< Session2.BuildVersion(BuildVersion.data(), int32(BuildVersion.size()))
		<< Session2.Platform(Platform.data(), int32(Platform.size()))
		<< Session2.CommandLine(CommandLine.data(), int32(CommandLine.size()))
		<< Session2.ConfigurationType(uint8(Variant));
}

} // namespace trace

#endif // TRACE_IMPLEMENT



// {{{1 cpu-header /////////////////////////////////////////////////////////////

TRACE_CHANNEL_EXTERN(CpuChannel)

namespace trace {

enum CpuScopeFlags : int32
{
	CpuFlush = 1 << 0,
};

struct TraceCpuScope
{
			~TraceCpuScope();
	void	Enter(int32 ScopeId, int32 Flags=0);
	int32	_ScopeId = 0;
};

int32	ScopeNew(const std::string_view& Name);

class	Lane;
bool	LaneIsTracing();
Lane*	LaneNew(const std::string_view& Name);
void	LaneDelete(Lane* Handle);
void	LaneEnter(Lane* Handle, int32 ScopeId);
void	LaneLeave();

} // namespace trace

#define TRACE_CPU_SCOPE(name, ...) \
	trace::TraceCpuScope TRACE_PRIVATE_UNIQUE_VAR(cpu_scope); \
	if (CpuChannel) { \
		using namespace std::literals; \
		static int32 TRACE_PRIVATE_UNIQUE_VAR(scope_id); \
		if (0 == TRACE_PRIVATE_UNIQUE_VAR(scope_id)) \
			TRACE_PRIVATE_UNIQUE_VAR(scope_id) = trace::ScopeNew(name##sv); \
		TRACE_PRIVATE_UNIQUE_VAR(cpu_scope).Enter(TRACE_PRIVATE_UNIQUE_VAR(scope_id), ##__VA_ARGS__); \
	} \
	do {} while (0)

// {{{1 cpu-source /////////////////////////////////////////////////////////////

#if TRACE_IMPLEMENT

TRACE_CHANNEL_DEFINE(CpuChannel)

namespace trace {
namespace detail {

TRACE_EVENT_BEGIN(CpuProfiler, EventSpec, NoSync|Important)
	TRACE_EVENT_FIELD(uint32, Id)
	TRACE_EVENT_FIELD(trace::AnsiString, Name)
TRACE_EVENT_END()

TRACE_EVENT_BEGIN(CpuProfiler, NextBatchContext, NoSync)
	TRACE_EVENT_FIELD(uint16, ThreadId)
TRACE_EVENT_END()

TRACE_EVENT_BEGIN(CpuProfiler, EventBatch, NoSync)
	TRACE_EVENT_FIELD(uint8[], Data)
TRACE_EVENT_END()

////////////////////////////////////////////////////////////////////////////////
static int32 encode32_7bit(int32 value, void* __restrict out)
{
	// Calculate the number of bytes
	int32 length = 1;
	length += (value >= (1 <<  7));
	length += (value >= (1 << 14));
	length += (value >= (1 << 21));

	// Add a gap every eigth bit for the continuations
	int32 ret = value;
	ret = (ret & 0x0000'3fff) | ((ret & 0x0fff'c000) << 2);
	ret = (ret & 0x007f'007f) | ((ret & 0x3f80'3f80) << 1);

	// Set the bits indicating another byte follows
	int32 continuations = 0x0080'8080;
	continuations >>= (sizeof(value) - length) * 8;
	ret |= continuations;

	::memcpy(out, &ret, sizeof(value));

	return length;
}

////////////////////////////////////////////////////////////////////////////////
static int32 encode64_7bit(int64 value, void* __restrict out)
{
	// Calculate the output length
	uint32 length = 1;
	length += (value >= (1ll <<  7));
	length += (value >= (1ll << 14));
	length += (value >= (1ll << 21));
	length += (value >= (1ll << 28));
	length += (value >= (1ll << 35));
	length += (value >= (1ll << 42));
	length += (value >= (1ll << 49));

	// Add a gap every eigth bit for the continuations
	int64 ret = value;
	ret = (ret & 0x0000'0000'0fff'ffffull) | ((ret & 0x00ff'ffff'f000'0000ull) << 4);
	ret = (ret & 0x0000'3fff'0000'3fffull) | ((ret & 0x0fff'c000'0fff'c000ull) << 2);
	ret = (ret & 0x007f'007f'007f'007full) | ((ret & 0x3f80'3f80'3f80'3f80ull) << 1);

	// Set the bits indicating another byte follows
	int64 continuations = 0x0080'8080'8080'8080ull;
	continuations >>= (sizeof(value) - length) * 8;
	ret |= continuations;

	::memcpy(out, &ret, sizeof(value));

	return length;
}

////////////////////////////////////////////////////////////////////////////////
class ScopeBuffer
{
public:
	void		SetThreadId(uint32 Value) { ThreadIdOverride = Value; }
	void		Flush(bool Force);
	void		Enter(uint64 Timestamp, uint32 ScopeId, int32 Flag=0);
	void		Leave(uint64 Timestamp);

private:
	enum
	{
		BufferSize	= 256,
		Overflow	= 16,
		EnterLsb	= 1,
		LeaveLsb	= 0,
	};
	uint64		PrevTimestamp = 0;
	uint8*		Cursor = Buffer;
	uint32		ThreadIdOverride = 0;
	uint16		Depth = 0;
	uint8		Buffer[BufferSize];
};

////////////////////////////////////////////////////////////////////////////////
void ScopeBuffer::Flush(bool Force)
{
	using namespace detail;

	if (Cursor == Buffer)
		return;

	if (Depth > 0 && !Force && (Cursor <= (Buffer + BufferSize - Overflow)))
		return;

	if (ThreadIdOverride)
		TRACE_LOG(CpuProfiler, NextBatchContext, true)
			<< NextBatchContext.ThreadId(uint16(ThreadIdOverride));

	TRACE_LOG(CpuProfiler, EventBatch, true)
		<< EventBatch.Data(Buffer, uint32(ptrdiff_t(Cursor - Buffer)));

	PrevTimestamp = 0;
	Cursor = Buffer;
}

////////////////////////////////////////////////////////////////////////////////
void ScopeBuffer::Enter(uint64 Timestamp, uint32 ScopeId, int32 Flags)
{
	Timestamp -= PrevTimestamp;
	PrevTimestamp += Timestamp;
	Cursor += encode64_7bit((Timestamp) << 1 | EnterLsb, Cursor);
	Cursor += encode32_7bit(ScopeId, Cursor);
	Depth++;

	bool ShouldFlush = (Flags & CpuScopeFlags::CpuFlush);
	Flush(ShouldFlush);
}

////////////////////////////////////////////////////////////////////////////////
void ScopeBuffer::Leave(uint64 Timestamp)
{
	Timestamp -= PrevTimestamp;
	PrevTimestamp += Timestamp;
	Cursor += encode64_7bit((Timestamp << 1) | LeaveLsb, Cursor);
	Depth--;

	Flush(false);
}



////////////////////////////////////////////////////////////////////////////////
class ThreadBuffer
{
public:
	static void	Enter(uint64 Timestamp, uint32 ScopeId, int32 Flags);
	static void	Leave(uint64 Timestamp);

private:
				~ThreadBuffer();
	ScopeBuffer	Inner;

	static thread_local ThreadBuffer TlsInstance;
};

thread_local ThreadBuffer ThreadBuffer::TlsInstance;

////////////////////////////////////////////////////////////////////////////////
inline void	ThreadBuffer::Enter(uint64 Timestamp, uint32 ScopeId, int32 Flags)
{
	TlsInstance.Inner.Enter(Timestamp, ScopeId, Flags);
}

////////////////////////////////////////////////////////////////////////////////
inline void	ThreadBuffer::Leave(uint64 Timestamp)
{
	TlsInstance.Inner.Leave(Timestamp);
}

////////////////////////////////////////////////////////////////////////////////
ThreadBuffer::~ThreadBuffer()
{
	Inner.Flush(true);
}



////////////////////////////////////////////////////////////////////////////////
class Lane
{
public:
				Lane(const std::string_view& Name);
				~Lane();
	void		Enter(int32 ScopeId);
	void		Leave();

private:
	ScopeBuffer	Buffer;
	uint32		Id;
};

////////////////////////////////////////////////////////////////////////////////
Lane::Lane(const std::string_view& Name)
: Id(ScopeNew(Name))
{
	Buffer.SetThreadId(Id);
}

////////////////////////////////////////////////////////////////////////////////
Lane::~Lane()
{
	Buffer.Flush(true);
}

////////////////////////////////////////////////////////////////////////////////
void Lane::Enter(int32 ScopeId)
{
	uint64 Timestamp = detail::TimeGetTimestamp();
	Buffer.Enter(Timestamp, ScopeId);
	Buffer.Flush(false);
}

////////////////////////////////////////////////////////////////////////////////
void Lane::Leave()
{
	uint64 Timestamp = detail::TimeGetTimestamp();
	Buffer.Leave(Timestamp);
	Buffer.Flush(false);
}

} // namespace detail



////////////////////////////////////////////////////////////////////////////////
int32 ScopeNew(const std::string_view& Name)
{
	using namespace detail;

	static int32 volatile NextSpecId = 1;
	int32 SpecId = AtomicAddRelaxed(&NextSpecId, 1);

	uint32 NameSize = uint32(Name.size());
	TRACE_LOG(CpuProfiler, EventSpec, true, NameSize)
		<< EventSpec.Id(uint32(SpecId))
		<< EventSpec.Name(Name.data(), NameSize);

	return SpecId;
}



////////////////////////////////////////////////////////////////////////////////
class Lane
	: public detail::Lane
{
	using detail::Lane::Lane;
};

bool	LaneIsTracing()							{ return bool(CpuChannel); }
Lane*	LaneNew(const std::string_view& Name)	{ return new Lane(Name); }
void	LaneDelete(Lane* Handle)				{ delete Handle; }
void	LaneEnter(Lane* Handle, int32 ScopeId)	{ Handle->Enter(ScopeId); }
void	LaneLeave(Lane* Handle)					{ Handle->Leave(); }



////////////////////////////////////////////////////////////////////////////////
TraceCpuScope::~TraceCpuScope()
{
	using namespace detail;

	if (!_ScopeId)
		return;

	uint64 Timestamp = TimeGetTimestamp();
	ThreadBuffer::Leave(Timestamp);
}

////////////////////////////////////////////////////////////////////////////////
void TraceCpuScope::Enter(int32 ScopeId, int32 Flags)
{
	using namespace detail;

	_ScopeId = ScopeId;
	uint64 Timestamp = TimeGetTimestamp();
	ThreadBuffer::Enter(Timestamp, ScopeId, Flags);
}

} // namespace trace

#endif // TRACE_IMPLEMENT



// {{{1 fmt-args-header ////////////////////////////////////////////////////////

namespace trace::detail {

template <typename T> concept IsIntegral	= std::is_integral<T>::value;
template <typename T> concept IsFloat		= std::is_floating_point<T>::value;

////////////////////////////////////////////////////////////////////////////////
class ArgPacker
{
public:
	template <typename... Types> ArgPacker(Types... Args);
	const uint8*	GetData() const { return Buffer; }
	uint32			GetSize() const { return uint32(ptrdiff_t(Cursor - Buffer)); }

private:
	template <typename T> struct TypeId {};
	template <typename T> requires IsIntegral<T>	struct TypeId<T> { enum { Value = 1 << 6 }; };
	template <typename T> requires IsFloat<T>		struct TypeId<T> { enum { Value = 2 << 6 }; };

	enum {
		BufferSize	= 512,
		TypeIdStr	= 3 << 6,
	};
	template <typename Type> uint32					PackValue(Type&& Value);
	template <typename Type, typename... U> void	Pack(Type&& Value, U... Next);
	void			Pack() {}
	uint8*			Cursor = Buffer;
	const uint8*	End = Buffer + BufferSize;
	uint8			Buffer[BufferSize];
};

////////////////////////////////////////////////////////////////////////////////
template <typename Type>
uint32 ArgPacker::PackValue(Type&& Value)
{
	uint32 Size = sizeof(Type);
	if (Cursor + Size > End)
		return 0;

	std::memcpy(Cursor, &Value, sizeof(Type));
	Cursor += Size;
	return TypeId<Type>::Value | sizeof(Type);
}

////////////////////////////////////////////////////////////////////////////////
template <typename... Types>
ArgPacker::ArgPacker(Types... Args)
{
	static_assert(sizeof...(Args) <= 0xff);

	int32 ArgCount = sizeof...(Args);
	if (ArgCount == 0)
		return;

	*Cursor = uint8(ArgCount);
	Cursor += ArgCount + 1;

	Pack(std::forward<Types>(Args)...);
}

////////////////////////////////////////////////////////////////////////////////
template <typename T, typename... U>
void ArgPacker::Pack(T&& Value, U... Next)
{
	uint8* TypeCursor = Buffer + Buffer[0] - sizeof...(Next);
	if ((*TypeCursor = uint8(PackValue(std::forward<T>(Value)))) != 0)
		return Pack(std::forward<U>(Next)...);

	while (++TypeCursor < Buffer + Buffer[0] + 1)
		*TypeCursor = 0;
}

} // namespace trace::detail

// {{{1 log-header /////////////////////////////////////////////////////////////

TRACE_CHANNEL_EXTERN(LogChannel)

namespace trace::detail {

////////////////////////////////////////////////////////////////////////////////
void	LogMessageImpl(int32 Id, const uint8* ParamBuffer, int32 ParamSize);
int32	LogMessageNew(const std::string_view& Format, const std::string_view& File, int32 Line);

////////////////////////////////////////////////////////////////////////////////
template <typename... Types>
void LogMessage(int32 Id, Types&&... Args)
{
	ArgPacker Packer(std::forward<Types>(Args)...);
	LogMessageImpl(Id, Packer.GetData(), Packer.GetSize());
}

} // namespace trace::detail

////////////////////////////////////////////////////////////////////////////////
#define TRACE_LOG_MESSAGE(format, ...) \
	if (LogChannel) { \
		using namespace std::literals; \
		static int32 message_id; \
		if (message_id == 0) \
			message_id = trace::detail::LogMessageNew( \
				format##sv, \
				TRACE_PRIVATE_CONCAT(__FILE__, sv), \
				__LINE__); \
		trace::detail::LogMessage(message_id, ##__VA_ARGS__); \
	} \
	do {} while (0)

// {{{1 log-source /////////////////////////////////////////////////////////////

#if TRACE_IMPLEMENT

TRACE_CHANNEL_DEFINE(LogChannel)

namespace trace::detail {

////////////////////////////////////////////////////////////////////////////////
#if 0
TRACE_EVENT_BEGIN(Logging, LogCategory, NoSync|Important)
	TRACE_EVENT_FIELD(const void*, CategoryPointer)
	TRACE_EVENT_FIELD(uint8, DefaultVerbosity)
	TRACE_EVENT_FIELD(trace::AnsiString, Name)
TRACE_EVENT_END()
#endif

TRACE_EVENT_BEGIN(Logging, LogMessageSpec, NoSync|Important)
	TRACE_EVENT_FIELD(uint32, LogPoint)
	//TRACE_EVENT_FIELD(uint16, CategoryPointer)
	TRACE_EVENT_FIELD(uint16, Line)
	TRACE_EVENT_FIELD(trace::AnsiString, FileName)
	TRACE_EVENT_FIELD(trace::AnsiString, FormatString)
TRACE_EVENT_END()

TRACE_EVENT_BEGIN(Logging, LogMessage, NoSync)
	TRACE_EVENT_FIELD(uint32, LogPoint)
	TRACE_EVENT_FIELD(uint64, Cycle)
	TRACE_EVENT_FIELD(uint8[], FormatArgs)
TRACE_EVENT_END()

////////////////////////////////////////////////////////////////////////////////
void LogMessageImpl(int32 Id, const uint8* ParamBuffer, int32 ParamSize)
{
	uint64 Timestamp = TimeGetTimestamp();

	TRACE_LOG(Logging, LogMessage, true)
		<< LogMessage.LogPoint(Id)
		<< LogMessage.Cycle(Timestamp)
		<< LogMessage.FormatArgs(ParamBuffer, ParamSize);
}

////////////////////////////////////////////////////////////////////////////////
int32 LogMessageNew(
	const std::string_view& Format,
	const std::string_view& File,
	int32 Line)
{
	static int32 volatile NextId = 1;
	int32 Id = AtomicAddRelaxed(&NextId, 1);

	int32 DataSize = 0;
	DataSize += int32(Format.size());
	DataSize += int32(File.size());

	TRACE_LOG(Logging, LogMessageSpec, true, DataSize)
		<< LogMessageSpec.LogPoint(Id)
		<< LogMessageSpec.Line(uint16(Line))
		<< LogMessageSpec.FileName(File.data(), int32(File.size()))
		<< LogMessageSpec.FormatString(Format.data(), int32(Format.size()));

	return Id;
}

} // namespace trace::detail

#endif // TRACE_IMPLEMENT



// {{{1 bookmark-header ////////////////////////////////////////////////////////

namespace trace::detail {

////////////////////////////////////////////////////////////////////////////////
void	BookmarkImpl(uint32 Id, const uint8* ParamBuffer, uint32 ParamSize);
uint32	BookmarkNew(const std::string_view& Format, const std::string_view& File, uint32 Line);

template <typename... Types>
void Bookmark(uint32 Id, Types&&... Args)
{
	ArgPacker Packer(std::forward<Types>(Args)...);
	BookmarkImpl(Id, Packer.GetData(), Packer.GetSize());
}

} // namespace trace::detail

////////////////////////////////////////////////////////////////////////////////
#define TRACE_BOOKMARK(format, ...) \
	if (LogChannel) { \
		using namespace std::literals; \
		static int32 bookmark_id; \
		if (bookmark_id == 0) \
			bookmark_id = trace::detail::BookmarkNew( \
				format##sv, \
				TRACE_PRIVATE_CONCAT(__FILE__, sv), \
				__LINE__); \
		trace::detail::Bookmark(bookmark_id, ##__VA_ARGS__); \
	} \
	do {} while (0)

// {{{1 bookmark-source ////////////////////////////////////////////////////////

#if TRACE_IMPLEMENT

namespace trace::detail {

////////////////////////////////////////////////////////////////////////////////
TRACE_EVENT_BEGIN(Misc, BookmarkSpec, NoSync|Important)
	TRACE_EVENT_FIELD(uint32, BookmarkPoint)
	TRACE_EVENT_FIELD(int32, Line)
	TRACE_EVENT_FIELD(trace::AnsiString, FormatString)
	TRACE_EVENT_FIELD(trace::AnsiString, FileName)
TRACE_EVENT_END()

TRACE_EVENT_BEGIN(Misc, Bookmark, NoSync)
	TRACE_EVENT_FIELD(uint64, Cycle)
	TRACE_EVENT_FIELD(uint32, BookmarkPoint)
	TRACE_EVENT_FIELD(uint8[], FormatArgs)
TRACE_EVENT_END()

////////////////////////////////////////////////////////////////////////////////
void BookmarkImpl(uint32 Id, const uint8* ParamBuffer, uint32 ParamSize)
{
	uint64 Timestamp = TimeGetTimestamp();

	TRACE_LOG(Misc, Bookmark, true)
		<< Bookmark.Cycle(Timestamp)
		<< Bookmark.BookmarkPoint(Id)
		<< Bookmark.FormatArgs(ParamBuffer, ParamSize);
}

////////////////////////////////////////////////////////////////////////////////
uint32 BookmarkNew(const std::string_view& Format, const std::string_view& File, uint32 Line)
{
	uint32 Id = uint32(uintptr_t(Format.data()) >> 2);

	int32 DataSize = 0;
	DataSize += int32(Format.size());
	DataSize += int32(File.size());

	TRACE_LOG(Misc, BookmarkSpec, true, DataSize)
		<< BookmarkSpec.BookmarkPoint(Id)
		<< BookmarkSpec.Line(uint16(Line))
		<< BookmarkSpec.FormatString(Format.data(), int32(Format.size()))
		<< BookmarkSpec.FileName(File.data(), int32(File.size()));

	return Id;
}

} // namespace trace::detail

#endif // TRACE_IMPLEMENT

// }}}

/* vim: set noet ft=cpp foldlevel=1 foldmethod=marker : */
