// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Not included directly

#if !defined(TRACE_UE_COMPAT_LAYER)
#	if defined(__UNREAL__) && __UNREAL__
#		define TRACE_UE_COMPAT_LAYER	0
#	else
#		define TRACE_UE_COMPAT_LAYER	1
#	endif
#endif

#if TRACE_UE_COMPAT_LAYER

// platform defines
#define PLATFORM_ANDROID	0
#define PLATFORM_APPLE		0
#define PLATFORM_HOLOLENS	0
#define PLATFORM_MAC		0
#define PLATFORM_UNIX		0
#define PLATFORM_WINDOWS	0

#ifdef _WIN32
#	undef PLATFORM_WINDOWS
#	define PLATFORM_WINDOWS			1
#elif defined(__linux__)
#	undef  PLATFORM_UNIX
#	define PLATFORM_UNIX			1
#elif defined(__APPLE__)
#	undef  PLATFORM_MAC
#	define PLATFORM_MAC				1
#	undef  PLATFORM_APPLE
#	define PLATFORM_APPLE			1
#endif

// arch defines
#if defined(__amd64__) || defined(_M_X64)
#	define PLATFORM_CPU_X86_FAMILY	1
#	define PLATFORM_CPU_ARM_FAMILY	0
#	define PLATFORM_64BITS			1
#	define PLATFORM_CACHE_LINE_SIZE	64
#elif defined(__arm64__) || defined(_M_ARM64) || defined(_M_ARM64EC)
#	define PLATFORM_CPU_X86_FAMILY	0
#	define PLATFORM_CPU_ARM_FAMILY	1
#	define PLATFORM_64BITS			1
#	define PLATFORM_CACHE_LINE_SIZE	64
#else
#	error Unknown architecture
#endif

#if PLATFORM_CPU_X86_FAMILY
#	define PLATFORM_USE_SSE2_FOR_THREAD_YIELD 1
#endif

// external includes
#if defined(_MSC_VER)
#	define _ENABLE_EXTENDED_ALIGNED_STORAGE
#endif

#include <algorithm>
#include <cstddef>
#include <cstdint>
#include <cstring>
#include <string>
#include <string_view>
#include <utility>
#include <vector>

#if PLATFORM_WINDOWS
#	if !defined(WIN32_LEAN_AND_MEAN)
#		define WIN32_LEAN_AND_MEAN
#	endif
#	if !defined(NOGDI)
#		define NOGDI
#	endif
#	if !defined(NOMINMAX)
#		define NOMINMAX
#	endif
#	include <Windows.h>
#endif

// types
using uint8  = uint8_t;
using uint16 = uint16_t;
using uint32 = uint32_t;
using uint64 = uint64_t;

using int8	= int8_t;
using int16 = int16_t;
using int32 = int32_t;
using int64 = int64_t;

using UPTRINT = uintptr_t;
using PTRINT  = intptr_t;

using SIZE_T = size_t;

#if PLATFORM_WINDOWS
#	undef TEXT
#endif
#define TEXT(x)	x
#define TCHAR	ANSICHAR
using ANSICHAR	= char;
using WIDECHAR	= char16_t;

// keywords
#if defined(_MSC_VER)
#	define FORCENOINLINE	__declspec(noinline)
#	define FORCEINLINE		__forceinline
#else
#	define FORCENOINLINE	inline __attribute__((noinline))
#	define FORCEINLINE		inline __attribute__((always_inline))
#endif

#if defined(_MSC_VER)
#	define LIKELY(x)		x
#	define UNLIKELY(x)		x
#else
#	define LIKELY(x)		__builtin_expect(!!(x), 1)
#	define UNLIKELY(x)		__builtin_expect(!!(x), 0)
#endif

#define UE_ARRAY_COUNT(x)	(sizeof(x) / sizeof(x[0]))

// so/dll
#if defined(TRACE_DLL_EXPORT)
#	if PLATFORM_WINDOWS && defined(TRACE_DLL_EXPORT)
#		if TRACE_IMPLEMENT
#			define TRACELOG_API __declspec(dllexport)
#		else
#			define TRACELOG_API __declspec(dllimport)
#		endif
#	else
#		define TRACELOG_API		__attribute__ ((visibility ("default")))
#	endif
#else
#	define TRACELOG_API
#endif
#define TRACEANALYSIS_API		TRACELOG_API

#if !defined(IS_MONOLITHIC)
#	define IS_MONOLITHIC				0
#endif

// misc defines
#define TSAN_SAFE
#define THIRD_PARTY_INCLUDES_END
#define THIRD_PARTY_INCLUDES_START
#define TRACE_ENABLED					1
#define TRACE_PRIVATE_CONTROL_ENABLED	0
#define TRACE_PRIVATE_EXTERNAL_LZ4		1
#define UE_AUTORTFM_ALWAYS_OPEN
#define UE_LAUNDER(x)					(x)
#define UE_DEPRECATED(x, ...)
#define UE_TRACE_ENABLED				TRACE_ENABLED
#define PREPROCESSOR_JOIN(a,b)			PREPROCESSOR_JOIN_IMPL(a,b)
#define PREPROCESSOR_JOIN_IMPL(a,b)		a##b
#define DEFINE_LOG_CATEGORY_STATIC(...)
#define UE_LOG(...)
#define check(x)						do { if (!(x)) __debugbreak(); } while (0)
#define checkf(x, ...)					check(x)

#define PRAGMA_ENABLE_DEPRECATION_WARNINGS
#define PRAGMA_DISABLE_DEPRECATION_WARNINGS

#if !defined(UE_BUILD_SHIPPING)
#	define UE_BUILD_SHIPPING			1
#	define UE_TRACE_FULL_SHIPPING_ENABLED_UNSUPPORTED
#endif
#define UE_BUILD_TEST					0

#if defined(__has_builtin)
#	if __has_builtin(__builtin_trap)
#		define PLATFORM_BREAK()			do { __builtin_trap(); } while (0)
#	endif
#elif defined(_MSC_VER)
#	define PLATFORM_BREAK()				do { __debugbreak(); } while (0)
#endif

#if !defined(PLATFORM_BREAK)
#	define PLATFORM_BREAK()				do { *(int*)0x493 = 0x493; } while (0)
#endif

// api
template <typename T> inline auto Forward(T t)		{ return std::forward<T>(t); }
template <typename T> inline auto MoveTemp(T&& t)	{ return std::move<T>(t); }
template <typename T> inline void Swap(T& x, T& y)	{ return std::swap(x, y); }



template <typename T, typename ALLOCATOR=void>
struct TArray
	: public std::vector<T>
{
	using	 Super = std::vector<T>;
	using	 Super::Super;
	using	 Super::back;
	using	 Super::begin;
	using	 Super::clear;
	using	 Super::data;
	using	 Super::emplace_back;
	using	 Super::empty;
	using	 Super::end;
	using	 Super::erase;
	using	 Super::front;
	using	 Super::insert;
	using	 Super::pop_back;
	using	 Super::push_back;
	using	 Super::reserve;
	using	 Super::resize;
	using	 Super::size;
	void	 Add(const T& t)					{ push_back(t); }
	T&		 Add_GetRef(const T& t)				{ push_back(t); return back(); }
	size_t	 AddUninitialized(size_t n)			{ resize(size() + n); return size() - n; }
	void	 Emplace()							{ emplace_back(); }
	T&		 Emplace_GetRef()					{ emplace_back(); return back(); }
	void	 Insert(T const& t, size_t i)		{ insert(begin() + i, t); }
	void	 Push(const T& t)					{ push_back(t); }
	T		 Pop(bool Eh=true)					{ T t = back(); pop_back(); return t; }
	size_t	 Num() const						{ return size(); }
	void	 SetNum(size_t num)					{ resize(num); }
	void	 SetNumZeroed(size_t num)			{ resize(num, 0); }
	void	 SetNumUninitialized(size_t num)	{ resize(num); }
	void	 Reserve(size_t num)				{ reserve(num); }
	T&		 Last()								{ return back(); }
	T*		 GetData()							{ return data(); }
	T const* GetData() const					{ return data(); }
	bool	 IsEmpty() const					{ return empty(); }
	void	 Empty()							{ clear(); }
	void	 Reset()							{ clear(); }
	void	 RemoveAll(bool (*p)(T const&))		{ erase(remove_if(begin(), end(), p), end()); }
	void	 RemoveAllSwap(bool (*p)(T const&))	{ RemoveAll(p); }

	struct NotLess {
		bool operator () (T const& Lhs, T const& Rhs) const {
			return bool(Lhs < Rhs) == false;
		}
	};
	void	 Heapify()				{ std::make_heap(begin(), end(), NotLess()); }
	void	 HeapPush(T const& t)	{ Add(t); std::push_heap(begin(), end(), NotLess()); }
	void	 HeapPopDiscard()		{ std::pop_heap(begin(), end(), NotLess()); pop_back(); }
	T const& HeapTop() const		{ return front(); }
};

template <size_t S> struct TInlineAllocator {};

template <typename T>
struct TArrayView
{
				TArrayView() = default;
				TArrayView(T* data, size_t size) : _data(data), _size(size) {}
	T*			GetData()		{ return _data; }
	T const*	GetData() const	{ return _data; }
	size_t		Num() const		{ return _size; }
	T*			_data = nullptr;
	size_t		_size = 0;
	using		value_type = T;
};


template <typename T>
struct TStringViewAdapter
	: public std::basic_string_view<T>
{
	using		Super = std::basic_string_view<T>;
	using		Super::Super;
	using		Super::size;
	using		Super::data;
	size_t		Len() const		{ return size(); }
	const char*	GetData() const	{ return data(); }
};

struct FStringProxy
{
							FStringProxy() = default;
	template <typename T>	FStringProxy(size_t len, const T* ptr);
	template <typename T>	FStringProxy(const T& r) : FStringProxy(r.size(), r.data()) {}
	std::string				_str;
	const void*				_ptr;
	uint32_t				_len;
	uint32_t				_char_size;
	std::string_view		ToView();
};

template <typename T>
inline FStringProxy::FStringProxy(size_t len, const T* ptr)
: _ptr(ptr)
, _len(uint32_t(len))
, _char_size(sizeof(T))
{
}

inline std::string_view FStringProxy::ToView()
{
	if (_char_size != 1)
	{
		_char_size = 1;
		_str.reserve(_len);
		char* __restrict cursor = _str.data();
		for (uint32_t i = 0, n = _len; i < n; ++i)
			cursor[i] = char(((const char16_t*)_ptr)[i]);
		_ptr = cursor;
	}

	return std::string_view((const char*)_ptr, _len);
}

using FAnsiStringView	= TStringViewAdapter<char>;
using FWideStringView	= TStringViewAdapter<char16_t>;
using FString			= struct FStringProxy;
using ANSITEXTVIEW		= FAnsiStringView;



namespace UECompat
{

template <typename T>
inline T			Max(T const& a, T const& b)				{ return std::max(a, b); }
inline size_t		Strlen(const char* str)					{ return ::strlen(str); }
inline char*		Strtok(char* str, const char* t, void*)	{ return ::strtok(str, t); }
inline const char*	Strchr(const char* str, int32 c)		{ return ::strchr(str, c); }
inline size_t		Atoi(const char* str)					{ return ::atoi(str); }
inline void*		Malloc(size_t size)						{ return ::malloc(size); }
inline void*		Realloc(void* ptr, size_t size)			{ return ::realloc(ptr, size); }
inline void			Free(void* ptr)							{ return ::free(ptr); }
inline void			Memcpy(void* d, const void* s, size_t n){ ::memcpy(d, s, n); }

template <typename T, typename COMPARE=std::less<typename T::value_type>>
inline void StableSort(T& container, COMPARE&& compare=COMPARE())
{
	std::stable_sort(container.GetData(), container.GetData() + container.Num(),
		std::forward<COMPARE>(compare));
}

template <typename T, typename COMPARE=std::less<typename T::value_type>>
inline void Sort(T& container, COMPARE&& compare=COMPARE())
{
	std::sort(container.GetData(), container.GetData() + container.Num(),
		std::forward<COMPARE>(compare));
}

template <typename T, typename P>
inline void SortBy(T& container, P const& p)
{
	Sort(container, [&p] (typename T::value_type& lhs, typename T::value_type& rhs) {
		return p(lhs) < p(rhs);
	});
}

template <typename T, typename V, typename P>
inline int LowerBoundBy(T const& container, V const& v, P const& p)
{
	auto adapter = [&p] (typename T::value_type const& lhs, V const& v) {
		return p(lhs) < v;
	};
	auto* first = container.GetData();
	auto* last = first + container.Num();
	auto iter = std::lower_bound(first, last, v, adapter);
	return int(ptrdiff_t(iter - first));
}

} // namespace UeCompat

namespace Algo				= UECompat;
namespace FMath				= UECompat;
namespace FCStringAnsi		= UECompat;
namespace FPlatformString	= UECompat;
namespace FMemory			= UECompat;

namespace AutoRTFM
{

inline bool	IsClosed()				{ return false; }
inline void	PushOnAbortHandler(...)	{}
inline void	PopOnAbortHandler(...)	{}
inline void	Open(...)				{}

} // namespace AutoRTFM

#define ON_SCOPE_EXIT auto NoOp = [&] ()

#endif // TRACE_UE_COMPAT_LAYER

#include <cstring>
#include "lz4.h"

#if PLATFORM_WINDOWS
#	pragma warning(push)
#	pragma warning(disable : 4200) // zero-sized arrays
#	pragma warning(disable : 4201) // anonymous structs
#	pragma warning(disable : 4127) // conditional expr. is constant
#	pragma warning(disable : 4003) // not function-macro args
#endif

/* vim: set noet ft=cpp foldlevel=1 foldmethod=marker : */
